{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "AndroidFileServer",
	"Description": "Adds support for remote file management to Android projects.",
	"Category": "Android",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": false,
	"IsBetaVersion": false,
	"EnabledByDefault": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "AndroidFileServer",
			"Type": "Runtime",
			"LoadingPhase": "PreLoadingScreen",
                        "PlatformAllowList" : [ "Android", "Win64", "Mac", "Linux" ]
                },
                {
                        "Name" : "AndroidFileServerEditor",
                        "Type" : "Editor",
                        "LoadingPhase" : "PostEngineInit",
                        "PlatformAllowList" : [ "Win64", "Mac", "Linux" ]
                }
        ]
}
