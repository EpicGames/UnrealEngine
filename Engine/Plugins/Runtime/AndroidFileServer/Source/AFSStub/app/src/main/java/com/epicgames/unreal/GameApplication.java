package com.epicgames.unreal;

import android.app.Application;
import android.content.Context;
import android.content.res.Configuration;
import androidx.lifecycle.Lifecycle;
import androidx.lifecycle.LifecycleObserver;
import androidx.lifecycle.OnLifecycleEvent;
import androidx.lifecycle.ProcessLifecycleOwner;
import com.epicgames.unreal.IAndroidFileServerSetup;


public class GameApplication extends Application implements LifecycleObserver , IAndroidFileServerSetup {
	private static final Logger Log = new Logger("UE", "GameApp");

	private static boolean isForeground = false;

	public boolean AndroidFileServer_Init(String filename)
	{
		return false;
	}

	public boolean AndroidFileServer_Verify(String Token)
	{
		return true;
	}

	@Override
	public void onCreate() {
		super.onCreate();


		ProcessLifecycleOwner.get().getLifecycle().addObserver(this);

	}

	@Override
	public void attachBaseContext(Context base) {
		super.attachBaseContext(base);

	}

	@Override
	public void onLowMemory() {
		super.onLowMemory();

	}

	@Override
	public void onTrimMemory(int level) {
		super.onTrimMemory(level);

	}

	@Override
	public void onConfigurationChanged (Configuration newConfig) {
		super.onConfigurationChanged(newConfig);
		
	}

	@OnLifecycleEvent(Lifecycle.Event.ON_START)
	void onEnterForeground() {
		Log.verbose("App in foreground");
		isForeground = true;
	}

	@OnLifecycleEvent(Lifecycle.Event.ON_STOP)
	void onEnterBackground() {
		Log.verbose("App in background");
		isForeground = false;
	}

	@SuppressWarnings("unused")
	public static boolean isAppInForeground() {
		return isForeground;
	}

	public static boolean isAppInBackground() {
		return !isForeground;
	}
}
