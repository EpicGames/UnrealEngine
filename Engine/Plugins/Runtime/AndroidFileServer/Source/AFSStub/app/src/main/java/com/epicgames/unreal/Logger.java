package com.epicgames.unreal;

import android.text.TextUtils;
import android.util.Log;

public class Logger
{
	public interface ILoggerCallback
	{
		void LoggerCallback(String Level, String Tag, String Message);
	}

	private static ILoggerCallback mCallback = null;
	private final String mTag;
	private final String mSecondaryTag;
	private boolean bHasSecondaryTag;
	private String mFormattedTag;
	private String mFormattedMessageTag;
	
	private static boolean bAllowLogging			= true;
	@SuppressWarnings({"FieldCanBeLocal", "unused"})
	private static boolean bAllowExceptionLogging	= true;
	private static boolean bPrependSecondaryTag	= false;
	private static boolean bPrependChanged	= false;
	
	@SuppressWarnings({"unused", "WeakerAccess"})
	public static void RegisterCallback(ILoggerCallback callback)
	{
		mCallback = callback;
	}

	@SuppressWarnings("WeakerAccess")
	public static void SuppressLogs ()
	{
		bAllowLogging = bAllowExceptionLogging = false;
	}

	@SuppressWarnings({"unused", "WeakerAccess"})
	public static void prependSecondaryTag(boolean prependSecondaryTag) {
		bPrependSecondaryTag = prependSecondaryTag;
		bPrependChanged = true;
	}
	
	public Logger(String Tag)
	{
		this(Tag, "");
	}

	public Logger(String Tag, String secondaryTag)
	{
		mTag = Tag;
		if(secondaryTag == null) {
			mSecondaryTag = "";
		} else {
			mSecondaryTag = secondaryTag;
		}
		bHasSecondaryTag = !TextUtils.isEmpty(mSecondaryTag);
	}

	public void verbose(String Message)
	{
		if (bAllowLogging)
		{
			Log.v(getFormattedTag(), getFormattedMessage(Message));
		}
		if (mCallback != null)
		{
			mCallback.LoggerCallback("V/", getFormattedTag(), getFormattedMessage(Message));
		}
	}

	public void debug(String Message)
	{
		if (bAllowLogging)
		{
			Log.d(getFormattedTag(), getFormattedMessage(Message));
		}
		if (mCallback != null)
		{
			mCallback.LoggerCallback("D/", getFormattedTag(), getFormattedMessage(Message));
		}
	}
	
	public void warn(String Message)
	{
		if (bAllowLogging)
		{
			Log.w(getFormattedTag(), getFormattedMessage(Message));
		}
		if (mCallback != null)
		{
			mCallback.LoggerCallback("W/", getFormattedTag(), getFormattedMessage(Message));
		}
	}
	
	public void error(String Message)
	{
		if (bAllowLogging)
		{
			Log.e(getFormattedTag(), getFormattedMessage(Message));
		}
		if (mCallback != null)
		{
			mCallback.LoggerCallback("E/", getFormattedTag(), getFormattedMessage(Message));
		}
	}

	public void error(String Message, Throwable Throwable)
	{
		if (bAllowLogging)
		{
			Log.e(getFormattedTag(), getFormattedMessage(Message), Throwable);
		}
		if (mCallback != null)
		{
			mCallback.LoggerCallback("E/", getFormattedTag(), getFormattedMessage(Message));
		}
	}
	
	private String getFormattedTag()
	{
		if(mFormattedTag == null || bPrependChanged)
		{
			bPrependChanged = false;
			if (bHasSecondaryTag && bPrependSecondaryTag)
			{
				mFormattedTag = mTag + "-" + mSecondaryTag;
			}
			else
			{
				mFormattedTag = mTag;
			}
		}
		return mFormattedTag;
	}

	private String getFormattedMessage(String message)
	{
		if (bHasSecondaryTag && !bPrependSecondaryTag)
		{
			if(mFormattedMessageTag == null || bPrependChanged)
			{
				bPrependChanged = false;
				mFormattedMessageTag = "[" + mSecondaryTag + "] ";
			}
			return mFormattedMessageTag + message;
		}
		return message;
	}
}
