// Copyright Epic Games, Inc. All Rights Reserved.

package com.epicgames.unreal;

import android.app.Activity;
import android.content.pm.ApplicationInfo;
import android.content.pm.PackageManager;
import android.os.Bundle;
import android.util.Log;
import android.view.ViewGroup.LayoutParams;
import android.view.ViewGroup.MarginLayoutParams;
import android.widget.LinearLayout;

import com.epicgames.unreal.RemoteFileManager;

import java.io.BufferedReader;
import java.io.IOException;
import java.io.InputStreamReader;
import java.io.PrintWriter;
import java.net.ServerSocket;
import java.net.Socket;
import java.util.Set;

import static android.os.Environment.getExternalStorageDirectory;

public class SplashActivity extends Activity {
    private static final int SERVER_PORT = 57099;

	@Override
	protected void onCreate(Bundle savedInstanceState) {
		super.onCreate(savedInstanceState);

		MarginLayoutParams params = new MarginLayoutParams(LayoutParams.WRAP_CONTENT, LayoutParams.WRAP_CONTENT);
		params.setMargins(0, 0, 0, 0);
		LinearLayout activityLayout = new LinearLayout(this);
		setContentView(activityLayout, params);

		int port = SERVER_PORT;

		// make sure UnrealFileServer not already running
		boolean bUSBRunning = false;
		boolean bWiFiRunning = false;
				
		Set<Thread> threads = Thread.getAllStackTraces().keySet();
		for (Thread thread : threads) {
			String name = thread.getName();
			if (name.equals("UnrealAFS-USB"))
			{
				if (thread.isAlive())
				{
					bUSBRunning = true;
				}
			}
			else if (name.equals("UnrealAFS-WiFi"))
			{
				if (thread.isAlive())
				{
					bWiFiRunning = true;
				}
			}
		}

		String packageName = getPackageName();
		PackageManager pm = getPackageManager();

		int versionCode = 0;
		try
		{
			versionCode = pm.getPackageInfo(packageName, 0).versionCode;
		}
		catch (Exception e)
		{
			// if the above failed, then, we can't use obbs
		}

		String EngineVersion = "5.0.0";
		String ProjectName = packageName;
		ProjectName = ProjectName.substring(ProjectName.lastIndexOf('.') + 1);
		boolean IsShipping = false;
		boolean UseExternalFilesDir = false;
		boolean PublicLogFiles = false;
		try {
			ApplicationInfo ai = pm.getApplicationInfo(packageName, PackageManager.GET_META_DATA);
			Bundle bundle = ai.metaData;

			if (bundle.containsKey("com.epicgames.unreal.GameActivity.EngineVersion"))
			{
				EngineVersion = bundle.getString("com.epicgames.unreal.GameActivity.EngineVersion");
			}
			if (bundle.containsKey("com.epicgames.unreal.GameActivity.ProjectName"))
			{
				ProjectName = bundle.getString("com.epicgames.unreal.GameActivity.ProjectName");
			}
			if (bundle.containsKey("com.epicgames.unreal.GameActivity.BuildConfiguration"))
			{
				String Configuration = bundle.getString("com.epicgames.unreal.GameActivity.BuildConfiguration");
				IsShipping = Configuration.contains("Shipping");
			}
			if (bundle.containsKey("com.epicgames.unreal.GameActivity.bUseExternalFilesDir"))
			{
				UseExternalFilesDir = bundle.getBoolean("com.epicgames.unreal.GameActivity.bUseExternalFilesDir");
			}
			if (bundle.containsKey("com.epicgames.unreal.GameActivity.bPublicLogFiles"))
			{
				PublicLogFiles = bundle.getBoolean("com.epicgames.unreal.GameActivity.bPublicLogFiles");
			}
		}
		catch (PackageManager.NameNotFoundException | NullPointerException e)
		{
			Log.e("UE","Error when accessing application metadata", e);
		}

		String internal = getFilesDir().getAbsolutePath();
		String external = getExternalFilesDir(null).getAbsolutePath();
		String storage = getExternalStorageDirectory().getAbsolutePath();
		String obbdir = getObbDir().getAbsolutePath();

		if (!bUSBRunning)
		{
			RemoteFileManager fileManager = new RemoteFileManager(true, port, internal, external, storage, obbdir, packageName, versionCode, ProjectName, EngineVersion, IsShipping, PublicLogFiles);
			new Thread(fileManager, "UnrealAFS-USB").start();
		}
		if (!bWiFiRunning)
		{
			RemoteFileManager fileManager = new RemoteFileManager(false, port, internal, external, storage, obbdir, packageName, versionCode, ProjectName, EngineVersion, IsShipping, PublicLogFiles);
			new Thread(fileManager, "UnrealAFS-WiFi").start();
		}
    }
}