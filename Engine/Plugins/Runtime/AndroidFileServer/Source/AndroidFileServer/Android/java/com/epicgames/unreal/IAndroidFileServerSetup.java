// Copyright Epic Games, Inc. All Rights Reserved.

package com.epicgames.unreal;

public interface IAndroidFileServerSetup
{
	public boolean AndroidFileServer_Init(String filename);
	public boolean AndroidFileServer_Verify(String Token);
}
