// Copyright Epic Games, Inc. All Rights Reserved.

package com.epicgames.unreal;

import android.app.DownloadManager;
import android.util.Log;

import java.io.ByteArrayInputStream;
import java.io.ByteArrayOutputStream;
import java.io.BufferedInputStream;
import java.io.BufferedOutputStream;
import java.io.BufferedReader;
import java.io.BufferedWriter;
import java.io.File;
import java.io.FileReader;
import java.io.FileOutputStream;
import java.io.InterruptedIOException;
import java.io.IOException;
import java.io.InputStream;
import java.io.InputStreamReader;
import java.io.OutputStream;
import java.io.PrintWriter;
import java.net.InetAddress;
import java.net.InetSocketAddress;
import java.net.NetworkInterface;
import java.nio.ByteBuffer;
import java.nio.charset.StandardCharsets;
import java.nio.file.FileVisitResult;
import java.nio.file.FileVisitor;
import java.nio.file.Files;
import java.nio.file.FileAlreadyExistsException;
import java.nio.file.NoSuchFileException;
import java.nio.file.Path;
import java.nio.file.Paths;
import java.nio.file.SimpleFileVisitor;
import java.nio.file.StandardCopyOption;
import java.nio.file.StandardOpenOption;
import java.nio.file.attribute.BasicFileAttributes;
import java.nio.file.attribute.PosixFilePermission;
import java.nio.file.attribute.PosixFilePermissions;
import java.util.ArrayList;
import java.util.Collections;
import java.util.HashMap;
import java.util.List;
import java.util.Map;
import java.util.Set;
import java.util.zip.GZIPInputStream;
import java.util.zip.GZIPOutputStream;

import java.net.ServerSocket;
import java.net.Socket;
import java.net.SocketException;

public class RemoteFileManager implements Runnable
{
	private String TAG = "UEFS";
	private static final int SERVER_VERSION = 101;
	private static final int ACCEPT_TIMEOUT = 2000;		// milliseconds

	private static final int MAX_OVERFLOW_FILES = 32;

	private static final int Command_Terminate = 0;
	private static final int Command_Close = 1;
	private static final int Command_Info = 2;
	private static final int Command_Query = 3;
	private static final int Command_GetProp = 4;
	private static final int Command_SetBaseDir = 5;

	private static final int Command_DirExists = 10;
	private static final int Command_DirList = 11;
	private static final int Command_DirListFlat = 12;
	private static final int Command_DirCreate = 13;
	private static final int Command_DirDelete = 14;
	private static final int Command_DirDeleteRecurse = 15;

	private static final int Command_FileExists = 20;
	private static final int Command_FileDelete = 21;
	private static final int Command_FileCopy = 22;
	private static final int Command_FileMove = 23;
	private static final int Command_FileRead = 24;
	private static final int Command_FileWrite = 25;
	private static final int Command_FileWriteCompressed = 26;
	private static final int Command_FileWriteHandshake = 27;

	private String externalPackageDir;
	private String internalPackageDir;
	private String externalStorageDir;

	private String obbPackageDir;
	private String unrealPackageDir;
	private String projectPackageDir;
	private String enginePackageDir;
	private String gamePackageDir;
	private String logsPackageDir;
	private String savedPackageDir;
	private String logfile;
	private String commandfile;
	private String mainobb;
	private String patchobb;

	private String packageName;
	private int versionCode;
	private String projectName;
	private String engineVersion;
	private String unrealGame;

	private boolean bIsShipping = false;
	private boolean bPublicLogs = false;

	private HashMap<String, String> replacements;
	private String LastBaseDir = "";

	private boolean bLocalConnection;
	private int serverPort;
	private String ipAddress;

	//	private InputStream inputStream;
//	private OutputStream outputStream;
	private BufferedInputStream inputStream;
	private BufferedOutputStream outputStream;
	private long dataRemaining;
	private boolean ioException;

	public RemoteFileManager(boolean bLocal, int port, String internal, String external, String storage, String obbdir, String PackageName,
							 int VersionCode, String ProjectName, String EngineVersion, boolean IsShipping, boolean PublicLogs)
	{
		TAG = bLocal ? "UEFS" : "UEFSW";

		ipAddress = getIPAddress(false);
		bLocalConnection = bLocal;
		serverPort = port;

		externalPackageDir = external + "/";
		internalPackageDir = internal + "/";
		externalStorageDir = storage + "/";
		packageName = PackageName;
		versionCode = VersionCode;
		String versionString = "" + versionCode;

		boolean bIsUE4 = EngineVersion.startsWith("4.");
		projectName = ProjectName;
		engineVersion = EngineVersion;
		unrealGame = bIsUE4 ? "UE4Game/" : "UnrealGame/";
		bIsShipping = IsShipping;
		bPublicLogs = PublicLogs;

		obbPackageDir = obbdir + "/";
		mainobb = obbPackageDir + "main." + versionString + "." + packageName + ".obb";
		patchobb = obbPackageDir + "patch." + versionString + "." + packageName + ".obb";

		unrealPackageDir = (bIsShipping ? internalPackageDir : externalPackageDir) + unrealGame;
		projectPackageDir = unrealPackageDir + projectName + "/";
		enginePackageDir = projectPackageDir + "Engine/";
		gamePackageDir = projectPackageDir + projectName + "/";
		savedPackageDir = gamePackageDir + "Saved/";
		logsPackageDir = (bIsShipping ? (bPublicLogs ? externalPackageDir : internalPackageDir) : externalPackageDir) +
				unrealGame + projectName + "/" + projectName + "/Saved/Logs/";

		commandfile = projectPackageDir + (bIsUE4 ? "UE4CommandLine.txt" : "UECommandLine.txt");
		logfile = logsPackageDir + projectName + ".log";

		replacements = new HashMap<String, String>();
		replacements.put("^ext/", externalPackageDir);
		replacements.put("^int/", internalPackageDir);
		replacements.put("^storage/", externalStorageDir);

		replacements.put("^packagename", packageName);
		replacements.put("^version", versionString);
		replacements.put("^obb", obbPackageDir);
		replacements.put("^mainobb", mainobb);
		replacements.put("^patchobb", patchobb);

		for (int overflowIndex=1; overflowIndex <= MAX_OVERFLOW_FILES; overflowIndex++)
		{
			String overflowobb = obbPackageDir + "overflow" + overflowIndex + "." + versionString + "." + packageName + ".obb";
			replacements.put("^overflow" + overflowIndex + "obb", overflowobb);
		}

		replacements.put("^engineversion", engineVersion);
		replacements.put("^projectname", projectName);
		replacements.put("^unreal/", unrealPackageDir);
		replacements.put("^project/", projectPackageDir);
		replacements.put("^engine/", enginePackageDir);
		replacements.put("^game/", gamePackageDir);
		replacements.put("^saved/", savedPackageDir);
		replacements.put("^logs/", logsPackageDir);

		replacements.put("^commandfile", commandfile);
		replacements.put("^logfile", logfile);
		replacements.put("^ip", ipAddress);

		LastBaseDir = "";
	}

	String getIPAddress(boolean getIPv6)
	{
		try
		{
			List<NetworkInterface> interfaces = Collections.list(NetworkInterface.getNetworkInterfaces());
			for (NetworkInterface iface : interfaces)
			{
				List<InetAddress> addresses = Collections.list(iface.getInetAddresses());
				for (InetAddress address : addresses) {
					if (!address.isLoopbackAddress()) {
						String result = address.getHostAddress();

						// IP6 contains :'s
						boolean bIPv6 = result.indexOf(':') >= 0;
						if (bIPv6) {
							if (getIPv6) {
								// remove zone suffix if present
								int zoneIndex = result.indexOf('%');
								return zoneIndex < 0 ? result : result.substring(0, zoneIndex);
							}
						} else if (!getIPv6) {
							return result;
						}
					}
				}
			}
		}
		catch (Exception ex)
		{
		}
		return "";
	}

	private String getProp(String property)
	{
		// first try SystemProperties reflection
		try
		{
			java.lang.reflect.Method get = Class.forName("android.os.SystemProperties").getMethod("get", new Class[]{String.class});
			String propValue = (String)get.invoke(null, property);
			if (propValue != null)
			{
				return propValue;
			}
			return "";
		}
		catch (Exception e)
		{
		}

		// fall back to getprop command
		Process process = null;
		BufferedReader reader = null;
		String result = "";

		try
		{
			process = new ProcessBuilder().command("/system/bin/getprop", property).redirectErrorStream(true).start();
			reader = new BufferedReader(new InputStreamReader(process.getInputStream()));
			String line = reader.readLine();
			result = (line == null) ? "" : line;
		}
		catch (Exception e)
		{
			Log.d(TAG, "Unable to use getprop: " + e);
		}
		if (reader != null)
		{
			try
			{
				reader.close();
			}
			catch (IOException e)
			{
			}
		}
		if (process != null)
		{
			process.destroy();
		}
		return result;
	}

	void SendResponse(String results)
	{
//		Log.d(TAG, "Results: " + results);

		// make sure we send back UTF8
		byte[] rawBytes = results.getBytes(StandardCharsets.UTF_8);
		long size = rawBytes.length;
		long buffersize = size + 8;

		byte[] buffer = new byte[(int)buffersize];
		buffer[0] = (byte)(size & 255);
		buffer[1] = (byte)((size >> 8) & 255);
		buffer[2] = (byte)((size >> 16) & 255);
		buffer[3] = (byte)((size >> 24) & 255);
		buffer[4] = (byte)((size >> 32) & 255);
		buffer[5] = (byte)((size >> 40) & 255);
		buffer[6] = (byte)((size >> 48) & 255);
		buffer[7] = (byte)((size >> 56) & 255);

		System.arraycopy(rawBytes, 0, buffer, 8, (int)size);

		try {
			outputStream.write(buffer, 0, (int)buffersize);
			outputStream.flush();
		}
		catch (Exception e)
		{
			ioException = true;
		}
	}

	String FixupPath(String path)
	{
		for (Map.Entry<String, String> entry : replacements.entrySet())
		{
			String key = entry.getKey();
			String value = entry.getValue();

			if (path.startsWith(key)) {
				return entry.getValue() + path.substring(key.length());
			}
		}
		return path;
	}

	String FixupPathLastDir(String path, boolean bPreserve)
	{
		String oldLastBaseDir = LastBaseDir;

		if (path.startsWith("^^/"))
		{
			path = LastBaseDir + path.substring(3);
		}
		else if (path.startsWith("^-"))
		{
			int slashIndex = path.indexOf("/");
			if (slashIndex > 1)
			{
				int dropCount =  Integer.parseInt(path.substring(2, slashIndex)) + 1;
				int index;
				while ((dropCount-- > 0) && ((index = LastBaseDir.lastIndexOf("/")) >= 0))
				{
					LastBaseDir = LastBaseDir.substring(0, index);
				}
				path = LastBaseDir + path.substring(slashIndex);
			}
		}

		int lastSlashIndex = path.lastIndexOf("/");
		LastBaseDir = path.substring(0, lastSlashIndex + 1);

		if (bPreserve)
		{
			LastBaseDir = oldLastBaseDir;
		}

		return FixupPath(path);
	}

	public void CommandQuery(String params)
	{
		if (params.isEmpty())
		{
			ClearResponse();
			for (Map.Entry<String, String> entry : replacements.entrySet())
			{
				AppendResponse(entry.getKey() + "\t" + entry.getValue() + "\n");
			}
			AppendResponse("^^/\t" + LastBaseDir + "\n");
			SendResponse(response);
			return;
		}
		String request = params;
		String result = FixupPathLastDir(request, true);
		if (result.equals(request))
		{
			request = request + "/";
			result = FixupPathLastDir(request, true);
		}

		SendResponse(result.equals(request) ? "Invalid" : result);
	}

	public void CommandDirExists(String params)
	{
		Path path = Paths.get(FixupPath(params));

		SendResponse(Files.isDirectory(path) ? "true" : "false");
	}

	public String response;

	public void ClearResponse()
	{
		response = "";
	}

	public void AppendResponse(String data)
	{
		response += data;
	}

	public void CommandDirList(String params)
	{
		boolean recurse = false;
		boolean filesize = false;
		boolean attribs = false;
		ClearResponse();

		int optionsIndex = params.indexOf(":");
		if (optionsIndex > 0)
		{
			String options = params.substring(0, optionsIndex);
			params = params.substring(optionsIndex+1);

			if (options.contains("R"))
			{
				recurse = true;
			}
			if (options.contains("S"))
			{
				filesize = true;
			}
			if (options.contains("A"))
			{
				attribs = true;
			}
		}

		if (!params.endsWith("/"))
		{
			params += "/";
		}
		Path path = Paths.get(FixupPath(params));
		final String pathstring = path.toString();
		final int pathlen = path.toString().length() + 1;
		final boolean bRecursive = recurse;
		final boolean bFilesize = filesize;
		final boolean bAttributes = attribs;

		if (!Files.isDirectory(path))
		{
			SendResponse("");
			return;
		}

		try {
			Files.walkFileTree(path, new FileVisitor<Path>() {
				@Override
				public FileVisitResult preVisitDirectory(Path dir, BasicFileAttributes attrs) throws IOException {
					if (dir.toString().equals(pathstring))
					{
						return FileVisitResult.CONTINUE;
					}
					String attrib = "";
					String size = "";
					if (bAttributes)
					{
						try {
							Set<PosixFilePermission> permissions = Files.getPosixFilePermissions(dir);
							attrib = "d";
							attrib += (permissions.contains(PosixFilePermission.OWNER_READ)) ? "r" : "-";
							attrib += (permissions.contains(PosixFilePermission.OWNER_WRITE)) ? "w" : "-";
							attrib += (permissions.contains(PosixFilePermission.OWNER_EXECUTE)) ? "x" : "-";
							attrib += (permissions.contains(PosixFilePermission.GROUP_READ)) ? "r" : "-";
							attrib += (permissions.contains(PosixFilePermission.GROUP_WRITE)) ? "w" : "-";
							attrib += (permissions.contains(PosixFilePermission.GROUP_EXECUTE)) ? "x" : "-";
							attrib += (permissions.contains(PosixFilePermission.OTHERS_READ)) ? "r" : "-";
							attrib += (permissions.contains(PosixFilePermission.OTHERS_WRITE)) ? "w" : "-";
							attrib += (permissions.contains(PosixFilePermission.OTHERS_EXECUTE)) ? "x " : "- ";
						}
						catch (Exception e)
						{
						}
					}
					if (bFilesize) {
						try {
							size = Files.size(dir) + " ";
						} catch (Exception e) {
							size = "0 ";
						}
					}
					AppendResponse("d " + attrib + size + dir.toString().substring(pathlen) + "\n");
					return bRecursive ? FileVisitResult.CONTINUE : FileVisitResult.SKIP_SUBTREE;
				}

				@Override
				public FileVisitResult visitFile(Path file, BasicFileAttributes attrs) throws IOException {
					String attrib = "";
					String size = "";
					if (bAttributes)
					{
						try {
							Set<PosixFilePermission> permissions = Files.getPosixFilePermissions(file);
							attrib = "-";
							attrib += (permissions.contains(PosixFilePermission.OWNER_READ)) ? "r" : "-";
							attrib += (permissions.contains(PosixFilePermission.OWNER_WRITE)) ? "w" : "-";
							attrib += (permissions.contains(PosixFilePermission.OWNER_EXECUTE)) ? "x" : "-";
							attrib += (permissions.contains(PosixFilePermission.GROUP_READ)) ? "r" : "-";
							attrib += (permissions.contains(PosixFilePermission.GROUP_WRITE)) ? "w" : "-";
							attrib += (permissions.contains(PosixFilePermission.GROUP_EXECUTE)) ? "x" : "-";
							attrib += (permissions.contains(PosixFilePermission.OTHERS_READ)) ? "r" : "-";
							attrib += (permissions.contains(PosixFilePermission.OTHERS_WRITE)) ? "w" : "-";
							attrib += (permissions.contains(PosixFilePermission.OTHERS_EXECUTE)) ? "x " : "- ";
						}
						catch (Exception e)
						{
						}
					}
					if (bFilesize) {
						try {
							size = Files.size(file) + " ";
						} catch (Exception e) {
							size = "0 ";
						}
					}

					AppendResponse( "f " + attrib + size + file.toString().substring(pathlen) + "\n");
					return FileVisitResult.CONTINUE;
				}

				@Override
				public FileVisitResult visitFileFailed(Path file, IOException exc) throws IOException {
					return FileVisitResult.CONTINUE;
				}

				@Override
				public FileVisitResult postVisitDirectory(Path dir, IOException exc) throws IOException {
					return FileVisitResult.CONTINUE;
				}
			});
		}
		catch (IOException e)
		{
		}
		catch (Exception e)
		{
		}
		SendResponse(response);
	}

	private ArrayList<Path> directories;
	private void VisitDirectories(Path baseDir)
	{
		AppendResponse(baseDir.toString() + ":\n");

		directories = new ArrayList<Path>();
		final String pathstring = baseDir.toString();
		try {
			Files.walkFileTree(baseDir, new FileVisitor<Path>() {
				@Override
				public FileVisitResult preVisitDirectory(Path dir, BasicFileAttributes attrs) throws IOException {
					if (dir.toString().equals(pathstring))
					{
						return FileVisitResult.CONTINUE;
					}
					directories.add(dir);
					return FileVisitResult.SKIP_SUBTREE;
				}

				@Override
				public FileVisitResult visitFile(Path file, BasicFileAttributes attrs) throws IOException {
					AppendResponse( file.getFileName() + "\n");
					return FileVisitResult.CONTINUE;
				}

				@Override
				public FileVisitResult visitFileFailed(Path file, IOException exc) throws IOException {
					return FileVisitResult.CONTINUE;
				}

				@Override
				public FileVisitResult postVisitDirectory(Path dir, IOException exc) throws IOException {
					return FileVisitResult.SKIP_SUBTREE;
				}
			});
		}
		catch (IOException e)
		{
		}
		catch (Exception e)
		{
		}

		for (Path dir : directories)
		{
			VisitDirectories(dir);
		}
	}

	public void CommandDirListFlat(String params)
	{
		boolean recurse = true;
		ClearResponse();

		if (!params.endsWith("/"))
		{
			params += "/";
		}

		Path path = Paths.get(FixupPath(params));
		if (!Files.isDirectory(path))
		{
			SendResponse("");
			return;
		}
		VisitDirectories(path);
		SendResponse(response);
	}

	private boolean DirCreateRecursive(String params)
	{
		try
		{
			File file = new File(params);
			return file.mkdirs();
		}
		catch (Exception e)
		{
		}
		return false;
	}

	/*  Note: file.mkdirs() is faster
	private boolean DirCreateRecursive(String params)
	{
		Path path = Paths.get(params);
		int lastIndex = path.getNameCount();

		// first walk find existing root
		int rootIndex = lastIndex;
		while (rootIndex > 0)
		{
			if (Files.isDirectory(path.subpath(0, rootIndex)))
			{
				break;
			}
			rootIndex--;
		}

		// create each directory until end
		while (rootIndex < lastIndex)
		{
			Path dirPath = path.subpath(0, ++rootIndex);
			try
			{
				Path dir = Files.createDirectory(dirPath);
			}
			catch (FileAlreadyExistsException e)
			{
				// this is ok
			}
			catch (NoSuchFileException e)
			{
				return false;
			}
			catch (IOException e)
			{
				return false;
			}
		}
		return true;
	}
	*/

	public void CommandDirCreateRecursive(String params)
	{
		if (!DirCreateRecursive(FixupPath(params))) {
			SendResponse("false");
			return;
		}
		SendResponse("true");
	}

	public void CommandDirCreate(String params)
	{
		Path path = Paths.get(FixupPath(params));

		try
		{
			Path dir = Files.createDirectory(path);
		}
		catch (FileAlreadyExistsException e)
		{
			// this is ok
		}
		catch (NoSuchFileException e)
		{
			// part of the path likely doesn't exist
			CommandDirCreateRecursive(params);
			return;
		}
		catch (IOException e)
		{
			SendResponse("false");
			return;
		}
		SendResponse("true");
	}

	public void CommandDirDelete(String params)
	{
		Path path = Paths.get(FixupPath(params));

		if (!Files.isDirectory(path))
		{
			SendResponse("false");
			return;
		}

		try
		{
			Files.delete(path);
		}
		catch (IOException e)
		{
			SendResponse("false");
			return;
		}
		SendResponse("true");
	}

	public void CommandDirDeleteRecurse(String params)
	{
		Path path = Paths.get(FixupPath(params));

		if (!Files.isDirectory(path))
		{
			SendResponse("false");
			return;
		}

		try
		{
			Files.walkFileTree(path, new SimpleFileVisitor<Path>() {
				@Override
				public FileVisitResult visitFile(Path file, BasicFileAttributes attrs) throws IOException
				{
					Files.delete(file);
					return FileVisitResult.CONTINUE;
				}

				@Override
				public FileVisitResult postVisitDirectory(Path dir, IOException exc) throws IOException
				{
					Files.delete(dir);
					return FileVisitResult.CONTINUE;
				}
			});
		}
		catch (IOException e)
		{
			SendResponse("false");
			return;
		}

		SendResponse("true");
	}

	public void CommandFileExists(String params)
	{
		Path path = Paths.get(FixupPath(params));

		SendResponse(Files.exists(path) ? "true" : "false");
	}

	public void CommandFileDelete(String params)
	{
		Path path = Paths.get(FixupPath(params));
//Log.d(TAG, "FileDelete: " + path.toString());

		if (!Files.exists(path) || Files.isDirectory(path))
		{
			SendResponse("false");
			return;
		}

		try
		{
			Files.delete(path);
		}
		catch (IOException e)
		{
			SendResponse("false");
			return;
		}
		SendResponse("true");
	}

	public void CommandFileCopy(String source, String dest)
	{
		Path sourcePath = Paths.get(FixupPath(source));
		Path destPath = Paths.get(FixupPath(dest));

		try
		{
			Files.copy(sourcePath, destPath, StandardCopyOption.REPLACE_EXISTING);
		}
		catch (FileAlreadyExistsException e)
		{
			SendResponse("false");
			return;
		}
		catch (IOException e)
		{
			SendResponse("false");
			return;
		}

		SendResponse("true");
	}

	public void CommandFileMove(String source, String dest)
	{
		Path sourcePath = Paths.get(FixupPath(source));
		Path destPath = Paths.get(FixupPath(dest));

		try
		{
			Files.move(sourcePath, destPath, StandardCopyOption.REPLACE_EXISTING);
		}
		catch (IOException e)
		{
			SendResponse("false");
			return;
		}

		SendResponse("true");
	}

	private void SendSize(long size)
	{
		byte[] buffer = new byte[8];
		buffer[0] = (byte) (size & 255);
		buffer[1] = (byte) ((size >> 8) & 255);
		buffer[2] = (byte) ((size >> 16) & 255);
		buffer[3] = (byte) ((size >> 24) & 255);
		buffer[4] = (byte) ((size >> 32) & 255);
		buffer[5] = (byte) ((size >> 40) & 255);
		buffer[6] = (byte) ((size >> 48) & 255);
		buffer[7] = (byte) ((size >> 56) & 255);

		try {
			outputStream.write(buffer, 0, 8);
			outputStream.flush();
		} catch (Exception e) {
			ioException = true;
		}
	}

	public void CommandFileRead(String params)
	{
		Path path = Paths.get(FixupPathLastDir(params, false));
//Log.d(TAG, "FileRead: " + path.toString());

		if (Files.isDirectory(path))
		{
			SendSize(-1);
			return;
		}
		if (!Files.exists(path))
		{
			SendSize(-1);
			return;
		}

		try {
			long filesize = Files.size(path);

			// special case stream - cannot predetermine size
			if (filesize == 0 && path.toString().startsWith("/proc/"))
			{
				ClearResponse();
				try (BufferedReader reader = new BufferedReader(new FileReader(new File(path.toString()))))
				{
					String line;
					while ((line = reader.readLine()) != null)
					{
						AppendResponse(line + "\n");
					}
				}
				catch (IOException ie)
				{
					SendSize(-1);
					return;
				}
				SendResponse(response);
				return;
			}

			SendSize(filesize);

			int bufferSize = 65536;
			byte[] buffer = new byte[bufferSize];

			InputStream reader = Files.newInputStream(path);
			while (filesize > 0)
			{
				int chunk = filesize < bufferSize ? (int)filesize : bufferSize;
				int readBytes = reader.read(buffer, 0, chunk);
				if (readBytes > 0)
				{
					outputStream.write(buffer, 0, readBytes);
					filesize -= readBytes;
				}
			}
			outputStream.flush();
			reader.close();
		}
		catch (IOException e)
		{

		}
	}

	private void IgnoreData(long size)
	{
		try
		{
			inputStream.skip(size);
			dataRemaining -= size;
		}
		catch (Exception e)
		{
			ioException = true;
		}
	}

	public void CommandFileWrite(String params)
	{
		Path path = Paths.get(FixupPathLastDir(params, false));
//Log.d(TAG, "FileWrite: " + path.toString());

		long filesize = dataRemaining;

		while (true)
		{
			try (OutputStream writer = Files.newOutputStream(path))
			{
				int bufferSize = 65536 * 16;
				byte[] buffer = new byte[bufferSize];

				while (filesize > 0)
				{
					int chunk = filesize < bufferSize ? (int)filesize : bufferSize;
					int readBytes = inputStream.read(buffer, 0, chunk);
					if (readBytes == -1)
					{
						writer.close();
						ioException = true;
						return;
					}
					if (readBytes > 0)
					{
						writer.write(buffer, 0, readBytes);
						filesize -= readBytes;
						dataRemaining -= readBytes;
					}
				}
				break;
			}
			catch (NoSuchFileException fe)
			{
				int lastIndex = path.getNameCount();
				if (!DirCreateRecursive(path.subpath(0, lastIndex - 1).toString()))
				{
					IgnoreData(dataRemaining);
					//Log.d(TAG, "FileWrite failed (NoSuchFileException): " + path.toString(), fe);
//					SendResponse("false");
					return;
				}
			}
			catch (Exception e)
			{
				IgnoreData(dataRemaining);
				Log.d(TAG, "FileWrite failed: " + path.toString(), e);
//				SendResponse("false");
				return;
			}
		}

//		SendResponse("true");
	}

	public void CommandFileWriteHandshake(byte[] timestamp, String params)
	{
		String fixedFilename = FixupPathLastDir(params, false);
		Path path = Paths.get(fixedFilename);
//Log.d(TAG, "FileWriteHandshake: " + path.toString());
		long filesize = dataRemaining;

		if (Files.isDirectory(path))
		{
			Log.d(TAG, "FileWriteHandshake failed: " + path.toString());
			SendSize(-1);
			return;
		}

		// make sure the directory exists if file doesn't
		int lastIndex = path.getNameCount();
		Path basedir = path.subpath(0, lastIndex - 1);
		if (!Files.isDirectory(basedir))
		{
			if (!DirCreateRecursive(basedir.toString()))
			{
				SendSize(-1);
				return;
			}
		}

		int bufferSize = 65536 * 16;
		byte[] buffer = new byte[bufferSize];

		// check for inprogress file to resume
		Path inprogressDataPath = Paths.get(fixedFilename+"_inprogress_.data");
		Path inprogressTimePath = Paths.get(fixedFilename+"_inprogress_.time");
		long currentSize = 0;
		try (InputStream reader = Files.newInputStream(inprogressTimePath))
		{
			currentSize = Files.size(inprogressDataPath);

			// validate timestamp to confirm attempting to resume same file
			int readBytes = reader.read(buffer, 0, 8);

			if (readBytes != 8 || buffer[0] != timestamp[0] || buffer[1] != timestamp[1] && buffer[2] != timestamp[2] || buffer[3] != timestamp[3] ||
				buffer[4] != timestamp[4] || buffer[5] != timestamp[5] || buffer[6] != timestamp[6] || buffer[7] != timestamp[7])
			{
				currentSize = 0;
			}
		}
		catch (Exception e)
		{
			currentSize = 0;
		}

		// write timestamp file if new
		if (currentSize == 0)
		{
			try (OutputStream writer = Files.newOutputStream(inprogressTimePath))
			{
				writer.write(timestamp, 0, 8);
			}
			catch (Exception e)
			{
				Log.d(TAG, "FileWriteHandshake failed: " + path.toString(), e);
				SendSize(-1);
				return;
			}
		}

		// request data from currentSize (0 for full file)
		dataRemaining = filesize - currentSize;
		SendSize(currentSize);

		OutputStream writer = null;
		try
		{
			if (currentSize == 0)
			{
				writer = Files.newOutputStream(inprogressDataPath);
			}
			else
			{
				writer = Files.newOutputStream(inprogressDataPath, StandardOpenOption.APPEND);
			}
			
			while (dataRemaining > 0)
			{
				int chunk = dataRemaining < bufferSize ? (int)dataRemaining : bufferSize;
				int readBytes = inputStream.read(buffer, 0, chunk);
				if (readBytes == -1)
				{
					writer.close();
					ioException = true;
					SendResponse("false");
					return;
				}
				if (readBytes > 0)
				{
					writer.write(buffer, 0, readBytes);
					dataRemaining -= readBytes;
				}
			}
			writer.close();
		}
		catch (Exception e)
		{
			// clean up so start over next time
			try
			{
				if (writer != null)
				{
					writer.close();
				}
				Files.delete(inprogressTimePath);
				Files.delete(inprogressDataPath);
			}
			catch (Exception e2)
			{
			}

			IgnoreData(dataRemaining);
			Log.d(TAG, "FileWriteHandshake failed: " + path.toString(), e);
			SendResponse("false");
			return;
		}

		// finalize file (delete timestamp and move in progress data to correct filename)
		try
		{
			Files.delete(inprogressTimePath);
			Files.move(inprogressDataPath, path, StandardCopyOption.REPLACE_EXISTING);
		}
		catch (Exception e)
		{
			SendResponse("false");
			return;
		}
		SendResponse("true");
	}

	public void CommandFileWriteCompressedOodle(String params)
	{
		Path path = Paths.get(FixupPathLastDir(params.substring(1), false));
//Log.d(TAG, "FileWriteCompressed: " + path.toString());

		long filesize = dataRemaining;
		boolean bUSB = TAG == "UEFS";
		dataRemaining = 0;

		int bufferSize = 1048576;
		byte[] buffer = new byte[bufferSize + 3];

		int handle = nativeCreateFile(path.toString(), bufferSize, bUSB);
		if (handle == -1)
		{
			return;
		}

		byte[] header = new byte[3];

		try {
			while (filesize > 0) {
				int readBytes;
				int headerOffset = 0;
				int headerBytes = 3;
				while (headerBytes > 0) {
					readBytes = inputStream.read(header, headerOffset, headerBytes);
					if (readBytes == -1) {
						nativeCloseFile(handle, bUSB);
						ioException = true;
						return;
					}
					headerOffset += readBytes;
					headerBytes -= readBytes;
				}

				// zero indicates uncompressed 1M chunk
				int chunk = ((int) header[0] & 0xff) + (((int) header[1] & 0xff) << 8) + (((int) header[2] & 0xff) << 16);
				boolean bUncompressed = (chunk == 0);
				chunk = (bUncompressed ? 1048576 : chunk);
				int offset = 0;
				int remaining = chunk;
				if (bUncompressed) {
					while (remaining > 0) {
						readBytes = inputStream.read(buffer, offset, remaining);
						if (readBytes == -1) {
							nativeCloseFile(handle, bUSB);
							ioException = true;
							return;
						}
						filesize -= readBytes;
						offset += readBytes;
						remaining -= readBytes;
					}
					nativeWrite(handle, buffer, offset);
					continue;
				}

				remaining += 3;
				while (remaining > 0) {
					readBytes = inputStream.read(buffer, offset, remaining);
					if (readBytes == -1) {
						nativeCloseFile(handle, bUSB);
						ioException = true;
						return;
					}
					offset += readBytes;
					remaining -= readBytes;
				}

				// use Ooodle decompression
				int rawBytes = ((int) buffer[0] & 0xff) + (((int) buffer[1] & 0xff) << 8) + (((int) buffer[2] & 0xff) << 16);
				readBytes = nativeOodleDecompress(buffer, chunk, handle, rawBytes, bUSB);
				filesize -= readBytes;
			}
			nativeCloseFile(handle, bUSB);
		}
		catch (Exception e)
		{
			//IgnoreData(dataRemaining);
			//SendResponse("false");
			return;
		}

		//SendResponse("true");
	}

	public void CommandFileWriteCompressed(String params)
	{
		char CompressType = params.charAt(0);
		if (CompressType == 'O')
		{
			CommandFileWriteCompressedOodle(params);
			return;
		}

		Path path = Paths.get(FixupPathLastDir(params.substring(1), false));
//Log.d(TAG, "FileWriteCompressed: " + path.toString());

		long filesize = dataRemaining;
		dataRemaining = 0;

		int bufferSize = 1048576;
		byte[] buffer = new byte[bufferSize];
		byte[] decompbuffer = new byte[bufferSize];

		while (true)
		{
			try
			{
				byte[] header = new byte[3];

				OutputStream writer = Files.newOutputStream(path);
				while (filesize > 0)
				{
					int readBytes;
					int headerOffset = 0;
					int headerBytes = 3;
					while (headerBytes > 0)
					{
						readBytes = inputStream.read(header, headerOffset, headerBytes);
						if (readBytes == -1)
						{
							writer.close();
							ioException = true;
							return;
						}
						headerOffset += readBytes;
						headerBytes -= readBytes;
					}

					// zero indicates uncompressed 1M chunk
					int chunk = ((int)header[0] & 0xff) + (((int)header[1] & 0xff) << 8) + (((int)header[2] & 0xff) << 16);
					boolean bUncompressed = (chunk == 0);
					chunk = (bUncompressed ? 1048576 : chunk);
					int offset = 0;
					int remaining = chunk;
					if (bUncompressed)
					{
						while (remaining > 0)
						{
							readBytes = inputStream.read(buffer, offset, remaining);
							if (readBytes == -1)
							{
								writer.close();
								ioException = true;
								return;
							}
							filesize -= readBytes;
							offset += readBytes;
							remaining -= readBytes;
						}
						writer.write(buffer, 0, offset);
						continue;
					}

					// decompress chunk
					if (CompressType == 'Z')
					{
						while (remaining > 0)
						{
							readBytes = inputStream.read(buffer, offset, remaining);
							if (readBytes == -1)
							{
								writer.close();
								ioException = true;
								return;
							}
							offset += readBytes;
							remaining -= readBytes;
						}

						ByteArrayInputStream byteArrayStream = new ByteArrayInputStream(buffer, 0, chunk);
						try
						{
							GZIPInputStream gzipStream = new GZIPInputStream(byteArrayStream);
							while ((readBytes = gzipStream.read(decompbuffer)) != -1)
							{
								writer.write(decompbuffer, 0, readBytes);
								filesize -= readBytes;
							}
							gzipStream.close();
						}
						catch (IOException e)
						{
							writer.close();
							ioException = true;
							return;
						}
					}
					else
					{
						// unknown compression type !
						writer.close();
						ioException = true;
						return;
					}
				}
				writer.close();
				break;
			}
			catch (NoSuchFileException fe)
			{
				int lastIndex = path.getNameCount();
				if (!DirCreateRecursive(path.subpath(0, lastIndex - 1).toString()))
				{
					IgnoreData(dataRemaining);
					//SendResponse("false");
					return;
				}
			}
			catch (Exception e)
			{
				//IgnoreData(dataRemaining);
				//SendResponse("false");
				return;
			}
		}

		//SendResponse("true");
	}

	private String readString()
	{
		// 256 should handle most input, and usually only grow once if not
		int index = 0;
		int capacity = 256;
		byte[] stringBuffer = new byte[capacity];

		int value = -1;
		while (dataRemaining > 0)
		{
			dataRemaining--;
			try
			{
				value = inputStream.read();
				if (value == 0)
				{
					break;
				}

				// grow the buffer if at capacity
				if (index == capacity)
				{
					capacity += 256;
					byte[] newBuffer = new byte[capacity];
					System.arraycopy(stringBuffer, 0, newBuffer, 0, index);
					stringBuffer = newBuffer;
				}
				stringBuffer[index++] = (byte)value;
			}
			catch (IOException e)
			{
				ioException = true;
				return "";
			}
		}

		// convert from byte array to string as UTF8
		return new String(stringBuffer, 0, index, StandardCharsets.UTF_8);
	}

	private boolean handleCommand(int command)
	{
		String params;
		String dest;

		switch (command)
		{
			case Command_Info:
				LastBaseDir = "";

				byte[] version = new byte[10];
				version[0] = 2;	version[1] = 0;	version[2] = 0;	version[3] = 0;	version[4] = 0;	version[5] = 0;
				version[6] = 0; version[7] = 0;
				version[8] = SERVER_VERSION & 255;
				version[9] = SERVER_VERSION >> 8;
				try {
					outputStream.write(version, 0, 10);
					outputStream.flush();
				}
				catch (IOException e)
				{
					Log.d(TAG, "Exception: " + e.toString());
					ioException = true;
					return false;
				}
				catch (Exception e)
				{
					Log.d(TAG, "Exception: " + e.toString());
				}
				return true;

			case Command_Query:
				params = readString();
				if (ioException) break;
				CommandQuery(params);
				return true;

			case Command_GetProp:
				params = readString();
				if (ioException) break;
				SendResponse(getProp(params));
				return true;

			case Command_SetBaseDir:
				params = readString();
				if (ioException) break;
				LastBaseDir = params.endsWith("/") ? params.substring(0, LastBaseDir.length() - 1) : params;
				SendResponse("true");
				return true;

			case Command_DirExists:
				params = readString();
				if (ioException) break;
				CommandDirExists(params);
				return true;

			case Command_DirList:
				params = readString();
				if (ioException) break;
				CommandDirList(params);
				return true;

			case Command_DirListFlat:
				params = readString();
				if (ioException) break;
				CommandDirListFlat(params);
				return true;

			case Command_DirCreate:
				params = readString();
				if (ioException) break;
				CommandDirCreate(params);
				return true;

			case Command_DirDelete:
				params = readString();
				if (ioException) break;
				CommandDirDelete(params);
				return true;

			case Command_DirDeleteRecurse:
				params = readString();
				if (ioException) break;
				CommandDirDeleteRecurse(params);
				return true;

			case Command_FileExists:
				params = readString();
				if (ioException) break;
				CommandFileExists(params);
				return true;

			case Command_FileDelete:
				params = readString();
				if (ioException) break;
				CommandFileDelete(params);
				return true;

			case Command_FileCopy:
				params = readString();
				if (ioException) break;
				dest = readString();
				if (ioException) break;
				CommandFileCopy(params, dest);
				return true;

			case Command_FileMove:
				params = readString();
				if (ioException) break;
				dest = readString();
				if (ioException) break;
				CommandFileMove(params, dest);
				return true;

			case Command_FileRead:
				params = readString();
				if (ioException) break;
				CommandFileRead(params);
				return false;

			case Command_FileWrite:
				params = readString();
				if (ioException) break;
				CommandFileWrite(params);
				return true;

			case Command_FileWriteCompressed:
				params = readString();
				if (ioException) break;
				CommandFileWriteCompressed(params);
				return true;

			case Command_FileWriteHandshake:
				{
					byte[] timestampData = new byte[8];
					int readBytes = 0;
					try
					{
						readBytes = inputStream.read(timestampData, 0, 8);
					}
					catch (IOException e)
					{
						ioException = true;
					}
					if (ioException || readBytes != 8) break;
					dataRemaining -= 8;
					params = readString();
					if (ioException) break;
					CommandFileWriteHandshake(timestampData, params);
				}
				return true;
		}
		return false;
	}

	public void run()
	{
		boolean bRunning = true;

		String serverIP = bLocalConnection ? "localhost" : ipAddress;
		if (serverIP.equals(""))
		{
			Log.d(TAG, "Unable to start remote file server for WiFi");
			return;
		}

		Log.d(TAG, "Starting remote file server " + packageName);
		try
		{
			ServerSocket serverSocket = new ServerSocket();
			serverSocket.setSoTimeout(ACCEPT_TIMEOUT);
//			Log.d(TAG, "Current recv buffer size: " + serverSocket.getReceiveBufferSize().toString());
			serverSocket.setReceiveBufferSize(65536 * 4);
			serverSocket.bind(new InetSocketAddress(serverIP, serverPort));

			while (bRunning && !Thread.currentThread().isInterrupted())
			{
				ioException = false;
				Log.d(TAG, "Starting server listen: " + serverPort);
				Socket socket = null;
				while (true)
				{
					try
					{
						Socket waitSocket = serverSocket.accept();
						socket = waitSocket;
						break;
					}
					catch (InterruptedIOException e1)
					{
						if (Thread.currentThread().isInterrupted())
						{
							bRunning = false;
							break;
						}
					}
					catch (SocketException e1)
					{
					}
					catch (IOException e1)
					{
					}
				}
				if (!bRunning)
				{
					break;
				}
				if (socket == null)
				{
					continue;
				}
				Log.d(TAG, "Client connected: " + socket.toString());

				outputStream = new BufferedOutputStream(socket.getOutputStream());
				inputStream = new BufferedInputStream(socket.getInputStream());

				// process commands
				byte[] commandHeader = new byte[8];
				while (!ioException && !Thread.currentThread().isInterrupted())
				{
					int readBytes = inputStream.read(commandHeader, 0, 8);
					if (readBytes == -1)
					{
						break;
					}
					if (readBytes == 8)
					{
						int command = commandHeader[0]  + (commandHeader[1] << 8);
						long datasize = ((long)commandHeader[2] & 0xff) + (((long)commandHeader[3] & 0xff) << 8) +
								(((long)commandHeader[4] & 0xff) << 16) + (((long)commandHeader[5] & 0xff) << 24) +
								(((long)commandHeader[6] & 0xff) << 32) + (((long)commandHeader[7] & 0xff) << 40);

//Log.d(TAG, "Command: " + command);

						// handle termination request
						if (command == Command_Terminate)// && datasize == 0)
						{
							bRunning = false;
							break;
						}

						// handle close request
						if (command == Command_Close)// && datasize == 0)
						{
							break;
						}

						// handle other commands
						dataRemaining = datasize;
						if (!handleCommand(command))
						{
						}

						// ignore remaining data
						if (!ioException && dataRemaining > 0)
						{
							inputStream.skip(dataRemaining);
						}
					}
				}

				// close connection and listen again
				try
				{
					socket.close();
				}
				catch (IOException e5)
				{
				}

				Log.d(TAG, "Client connection closed");
			}
			try
			{
				serverSocket.close();
			}
			catch (Exception sse)
			{
			}
		}
		catch (IOException e)
		{
			Log.d(TAG, "Server Exception: " + e.toString());
		}

		Log.d(TAG, "Server terminated " + packageName);
	}

	public native int nativeOodleMemSizeNeeded(int compressor, int rawLen);
	public native int nativeCreateFile(String filename, int bufferSize, boolean bUSB);
	public native int nativeWrite(int handle, byte[] buffer, int bytes);
	public native int nativeCloseFile(int handle, boolean bUSB);
	public native int nativeOodleDecompress(byte[] compbuffer, int compsize, int handle, int decompsize, boolean bUSB);

//	static
//	{
//		System.loadLibrary("fileserver");
//	}
}
