// Copyright Epic Games, Inc. All Rights Reserved.

package com.epicgames.unreal;

import android.app.Activity;
import android.app.Application;
import android.app.Notification;
import android.app.NotificationManager;
import android.app.NotificationChannel;
import android.app.Service;
import android.content.Context;
import android.content.Intent;
import android.os.Bundle;
import android.util.Log;

import java.util.Set;

import com.epicgames.unreal.RemoteFileManagerService;
import com.epicgames.unreal.IAndroidFileServerSetup;


public class RemoteFileManagerActivity extends Activity {
    private static String TAG = "UEFS";
    private static final int SERVER_PORT = 57099;

	private static final String NOTIFICATION_CHANNEL_ID = "unreal-afs-notification-channel-id";
	private static final CharSequence NOTICATION_CHANNEL_NAME = "unreal-afs-notification-channel";

	@Override
	protected void onCreate(Bundle savedInstanceState) {
		super.onCreate(savedInstanceState);

		Intent intent = getIntent();
		String action = intent.getAction();
		if (action != null && action.equals("com.epicgames.unreal.RemoteFileManager.intent.COMMAND2")) {
			String cmd = intent.getStringExtra("cmd");
			String token = intent.getStringExtra("token");
			int port = intent.getIntExtra("port", SERVER_PORT);

			Log.d(TAG, "RemoteFileManagerActivity cmd: " + cmd + ", package = " + getPackageName());

			IAndroidFileServerSetup AppAFSInterface = (IAndroidFileServerSetup)getApplication();

			if ("init".equals(cmd) && AppAFSInterface != null)
			{
				AppAFSInterface.AndroidFileServer_Init(getFilesDir().getAbsolutePath() + "/afs.cache");
			}
			else if ("stop".equals(cmd))
			{
				stopAllThreads();
				stopService(new Intent(this, RemoteFileManagerService.class));
			}
			else if ("start".equals(cmd) && AppAFSInterface != null && AppAFSInterface.AndroidFileServer_Verify(token)) {

				// make a service intent with same extras
				Intent serviceIntent = new Intent(this, RemoteFileManagerService.class);
				serviceIntent.setAction(action);
				serviceIntent.putExtra("cmd", cmd);
				serviceIntent.putExtra("token", token);
				serviceIntent.putExtra("port", port);

				// make sure the notification channel exists (create it if not)
				NotificationManager notificationManager = (NotificationManager) getSystemService(NotificationManager.class);
				NotificationChannel channel = notificationManager.getNotificationChannel(NOTIFICATION_CHANNEL_ID);
				if (channel == null)
				{
					channel = new NotificationChannel(NOTIFICATION_CHANNEL_ID, NOTICATION_CHANNEL_NAME, NotificationManager.IMPORTANCE_LOW);
					channel.enableVibration(false);
					channel.enableLights(false);
					channel.setSound(null, null);
					channel.setShowBadge(false);
					channel.setLockscreenVisibility(Notification.VISIBILITY_SECRET);
					notificationManager.createNotificationChannel(channel);
				}

				// make sure UnrealFileServer not already running
				boolean bUSBRunning = false;
				boolean bWiFiRunning = false;
				
				Set<Thread> threads = Thread.getAllStackTraces().keySet();
				for (Thread thread : threads) {
					String name = thread.getName();
					if (name.equals("UnrealAFS-USB"))
					{
						if (thread.isAlive())
						{
							bUSBRunning = true;
						}
					}
					else if (name.equals("UnrealAFS-WiFi"))
					{
						if (thread.isAlive())
						{
							bWiFiRunning = true;
						}
					}
				}

				if (bUSBRunning && bWiFiRunning)
				{
					Log.d(TAG, "Both server threads already active on service");
				}
				else
				{
					// start the service
					getApplicationContext().startForegroundService(serviceIntent);
				}
			}
		}

		// now terminate
		finish();
    }

	private void stopAllThreads()
	{
		String packageName = getPackageName();
		// send stop request to threads
		Set<Thread> threads = Thread.getAllStackTraces().keySet();
		for (Thread thread : threads) {
			String name = thread.getName();
			if (name.equals("UnrealAFS-USB") || name.equals("UnrealAFS-WiFi"))
			{
				if (thread.isAlive())
				{
					Log.d(TAG, "Sent stop request to " + packageName + "/" + name);
					thread.interrupt();
				}
			}
		}
	}
}
