// Copyright Epic Games, Inc. All Rights Reserved.

package com.epicgames.unreal;

import android.app.Service;
import android.content.Context;
import android.content.Intent;
import android.content.pm.ApplicationInfo;
import android.content.pm.PackageManager;
import android.os.Bundle;
import android.os.IBinder;
import android.util.Log;

import android.app.Notification;
import android.app.NotificationManager;
import android.app.NotificationChannel;
import android.app.PendingIntent;

import com.epicgames.unreal.IAndroidFileServerSetup;
import com.epicgames.unreal.RemoteFileManager;

import java.io.BufferedReader;
import java.io.IOException;
import java.io.InputStreamReader;
import java.io.PrintWriter;
import java.io.File;
import java.net.ServerSocket;
import java.net.Socket;
import java.util.Set;

import static android.os.Environment.getExternalStorageDirectory;

public class RemoteFileManagerService extends Service {
    private static String TAG = "UEFS";
    private static final int SERVER_PORT = 57099;

	private static final String NOTIFICATION_CHANNEL_ID = "unreal-afs-notification-channel-id";
	private static final CharSequence NOTICATION_CHANNEL_NAME = "unreal-afs-notification-channel";

	@Override
	public int onStartCommand(Intent intent, int flags, int startId)
	{
		NotificationManager notificationManager = (NotificationManager) getSystemService(NotificationManager.class);
		NotificationChannel channel = notificationManager.getNotificationChannel(NOTIFICATION_CHANNEL_ID);

		if (channel != null)
		{
			Intent notificationIntent = new Intent(this, RemoteFileManagerService.class);
			PendingIntent pendingIntent = PendingIntent.getActivity(this, 0, notificationIntent, PendingIntent.FLAG_IMMUTABLE);

			Notification.Builder notificationBuilder = new Notification.Builder(this, NOTIFICATION_CHANNEL_ID)
				.setSmallIcon(getNotificationIconID(this))
				.setContentTitle("Unreal Android File Server")
				.setContentText("Running AFS")
				.setContentIntent(pendingIntent)
				.setTicker("UE-AFS")
				.setOngoing(true);
			if (android.os.Build.VERSION.SDK_INT >= 31)
			{
				notificationBuilder.setForegroundServiceBehavior(Notification.FOREGROUND_SERVICE_DEFERRED);
			}

			startForeground(1, notificationBuilder.build());
		}
		else
		{
			Log.d(TAG, "Notification channel not available; service not started");
			stopForeground(Service.STOP_FOREGROUND_REMOVE);
			stopSelf(startId);
			return START_NOT_STICKY;
		}

		IAndroidFileServerSetup AppAFSInterface = (IAndroidFileServerSetup)getApplication();

		String action = intent.getAction();
		if (action != null && action.equals("com.epicgames.unreal.RemoteFileManager.intent.COMMAND2")) {
			String cmd = intent.getStringExtra("cmd");
			String token = intent.getStringExtra("token");
			int port = intent.getIntExtra("port", SERVER_PORT);
			if (AppAFSInterface != null && cmd != null && cmd.equals("start") && AppAFSInterface.AndroidFileServer_Verify(token)) {
				String packageName = getPackageName();
				PackageManager pm = getPackageManager();

				Log.d(TAG, "onStartCommand " + startId + " received start command " + packageName);

				// make sure UnrealFileServer not already running
				boolean bUSBRunning = false;
				boolean bWiFiRunning = false;
				
				Set<Thread> threads = Thread.getAllStackTraces().keySet();
				for (Thread thread : threads) {
					String name = thread.getName();
					if (name.equals("UnrealAFS-USB"))
					{
						if (thread.isAlive())
						{
							bUSBRunning = true;
						}
					}
					else if (name.equals("UnrealAFS-WiFi"))
					{
						if (thread.isAlive())
						{
							bWiFiRunning = true;
						}
					}
				}

				if (bUSBRunning && bWiFiRunning)
				{
					Log.d(TAG, "Both server threads already active");
					return START_REDELIVER_INTENT;
				}

				int versionCode = 0;
				try
				{
					if (android.os.Build.VERSION.SDK_INT >= 28)
					{
						versionCode = (int) pm.getPackageInfo(packageName, 0).getLongVersionCode();
					}
					else
					{
						versionCode = pm.getPackageInfo(packageName, 0).versionCode;
					}
				}
				catch (Exception e)
				{
					// if the above failed, then, we can't use obbs
				}

				String EngineVersion = "5.0.0";
				String ProjectName = packageName;
				ProjectName = ProjectName.substring(ProjectName.lastIndexOf('.') + 1);
				boolean IsShipping = false;
				boolean UseExternalFilesDir = false;
				boolean PublicLogFiles = false;
				try {
					ApplicationInfo ai = pm.getApplicationInfo(packageName, PackageManager.GET_META_DATA);
					Bundle bundle = ai.metaData;

					if (bundle.containsKey("com.epicgames.unreal.GameActivity.EngineVersion"))
					{
						EngineVersion = bundle.getString("com.epicgames.unreal.GameActivity.EngineVersion");
					}
					if (bundle.containsKey("com.epicgames.unreal.GameActivity.ProjectName"))
					{
						ProjectName = bundle.getString("com.epicgames.unreal.GameActivity.ProjectName");
					}
					if (bundle.containsKey("com.epicgames.unreal.GameActivity.BuildConfiguration"))
					{
						String Configuration = bundle.getString("com.epicgames.unreal.GameActivity.BuildConfiguration");
						IsShipping = Configuration.contains("Shipping");
					}
					if (bundle.containsKey("com.epicgames.unreal.GameActivity.bUseExternalFilesDir"))
					{
						UseExternalFilesDir = bundle.getBoolean("com.epicgames.unreal.GameActivity.bUseExternalFilesDir");
					}
					if (bundle.containsKey("com.epicgames.unreal.GameActivity.bPublicLogFiles"))
					{
						PublicLogFiles = bundle.getBoolean("com.epicgames.unreal.GameActivity.bPublicLogFiles");
					}
				}
				catch (PackageManager.NameNotFoundException | NullPointerException e)
				{
					Log.e(TAG, "Error when accessing application metadata", e);
				}

				String internal = getFilesDir().getAbsolutePath();
				String external = getExternalFilesDir(null).getAbsolutePath();
				File ExtStorageDir;
				if (android.os.Build.VERSION.SDK_INT >= 29)
				{
					// Derive external storage root from getExternalFilesDir path (no deprecated API needed).
					// external = .../Android/data/<pkg>/files; strip that suffix to get storage root.
					String AndroidSuffix = "/Android/data/" + packageName + "/files";
					if (external != null && external.endsWith(AndroidSuffix))
					{
						ExtStorageDir = new File(external.substring(0, external.length() - AndroidSuffix.length()));
					}
					else
					{
						Log.e(TAG, "Could not determine external storage root - using external files dir as fallback");
						ExtStorageDir = null;
					}
				}
				else
				{
					ExtStorageDir = android.os.Environment.getExternalStorageDirectory();
					if (ExtStorageDir == null)
					{
						Log.e(TAG, "getExternalStorageDirectory() returned null - using external files dir as fallback");
					}
				}
				String storage = (ExtStorageDir != null) ? ExtStorageDir.getAbsolutePath() : external;
				String obbdir = getObbDir().getAbsolutePath();

				if (!bUSBRunning)
				{
					RemoteFileManager fileManager = new RemoteFileManager(true, port, internal, external, storage, obbdir, packageName, versionCode, ProjectName, EngineVersion, IsShipping, PublicLogFiles);
					Thread usbThread = new Thread(fileManager, "UnrealAFS-USB");
					usbThread.setUncaughtExceptionHandler((thread, ex) -> Log.e(TAG, "Server thread " + thread.getName() + " crashed: " + ex.toString()));
					usbThread.start();
				}
				if (!bWiFiRunning)
				{
					RemoteFileManager fileManager = new RemoteFileManager(false, port, internal, external, storage, obbdir, packageName, versionCode, ProjectName, EngineVersion, IsShipping, PublicLogFiles);
					Thread wifiThread = new Thread(fileManager, "UnrealAFS-WiFi");
					wifiThread.setUncaughtExceptionHandler((thread, ex) -> Log.e(TAG, "Server thread " + thread.getName() + " crashed: " + ex.toString()));
					wifiThread.start();
				}

				return START_REDELIVER_INTENT;
			}
			else if (cmd != null && cmd.equals("start")) {
				// failed token verify - ignore it
				String packageName = getPackageName();
				Log.d(TAG, "onStartCommand " + startId + " received start command: " + packageName + " BAD TOKEN");
				return START_NOT_STICKY;
			}
			else if (cmd != null && cmd.equals("stop")) {
				String packageName = getPackageName();
				Log.d(TAG, "onStartCommand " + startId + " received stop command: " + packageName);
				stopAllThreads();
				stopForeground(Service.STOP_FOREGROUND_REMOVE);
				stopSelf(startId);
				return START_NOT_STICKY;
			}
		}

		Log.d(TAG, "onStartCommand received unknown command - NOT STARTING");
		return START_NOT_STICKY;
	}
	
	public static int getNotificationIconID(Context context)
	{
		int notificationIconID = context.getResources().getIdentifier("ic_notification_simple", "drawable", context.getPackageName());
		if (notificationIconID == 0)
		{
			notificationIconID = context.getResources().getIdentifier("ic_notification", "drawable", context.getPackageName());
		}
		if (notificationIconID == 0)
		{
			notificationIconID = context.getResources().getIdentifier("icon", "drawable", context.getPackageName());
		}
		return notificationIconID;
	}

	private void stopAllThreads()
	{
		String packageName = getPackageName();
		// send stop request to threads
		Set<Thread> threads = Thread.getAllStackTraces().keySet();
		for (Thread thread : threads) {
			String name = thread.getName();
			if (name.equals("UnrealAFS-USB") || name.equals("UnrealAFS-WiFi"))
			{
				if (thread.isAlive())
				{
					Log.d(TAG, "Sent stop request to " + packageName + "/" + name);
					thread.interrupt();
				}
			}
		}
	}

	@Override
	public IBinder onBind(Intent intent)
	{
		// binding not supported
		return null;
	}

	// Android 14 has a 6-hour runtime limit.  onTimeOut() is called and we stop the service rather
	// than having the system throw exception that shows up in crashes
	@Override
	public void onTimeout(int startId, int fgsType)
	{
		Log.w(TAG, "onTimeout: dataSync timeout reached, stopping service");
		stopAllThreads();
		stopForeground(Service.STOP_FOREGROUND_REMOVE);
		stopSelf(startId);
	}

	@Override
	public void onDestroy()
	{
		super.onDestroy();
//		String packageName = getPackageName();
//		Log.d(TAG, "onDestroy " + packageName);
	}

	@Override
	public void onCreate()
	{
		super.onCreate();
//		String packageName = getPackageName();
//		Log.d(TAG, "onCreate " + packageName);
    }
}
