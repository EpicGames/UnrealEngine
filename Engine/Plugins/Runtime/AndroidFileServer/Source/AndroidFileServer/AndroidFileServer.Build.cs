// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

public class AndroidFileServer : ModuleRules
{
	public AndroidFileServer(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;
		
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
			}
			);
			
		
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"CoreUObject",
				"Engine",
				"Slate",
				"SlateCore",
			}
			);
		
		
		if (Target.Platform == UnrealTargetPlatform.Android)
		{
			string PluginPath = Utils.MakePathRelativeTo(ModuleDirectory, Target.RelativeEnginePath);
			AdditionalPropertiesForReceipt.Add("AndroidPlugin", Path.Combine(PluginPath, "AndroidFileServer_UPL.xml"));

			PrivateDependencyModuleNames.AddRange(new string[] { "Launch" });
		}
	}
}
