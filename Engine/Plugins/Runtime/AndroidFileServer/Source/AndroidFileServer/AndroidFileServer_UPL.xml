<?xml version="1.0" encoding="utf-8"?>
<root xmlns:android="http://schemas.android.com/apk/res/android">
	<init>
		<log text="AndroidFileServer Plugin Init"/>
		<setBoolFromProperty result="bEnabled" ini="Engine" section="/Script/AndroidFileServerEditor.AndroidFileServerRuntimeSettings" property="bEnablePlugin" default="true"/>
		<setBoolFromProperty result="bAllowNetworkConnection" ini="Engine" section="/Script/AndroidFileServerEditor.AndroidFileServerRuntimeSettings" property="bAllowNetworkConnection" default="true"/>
		<setBoolFromProperty result="bIncludeInShipping" ini="Engine" section="/Script/AndroidFileServerEditor.AndroidFileServerRuntimeSettings" property="bIncludeInShipping" default="false"/>
		<setBoolFromProperty result="bAllowExternalStartInShipping" ini="Engine" section="/Script/AndroidFileServerEditor.AndroidFileServerRuntimeSettings" property="bAllowExternalStartInShipping" default="false"/>
		<setBoolFromProperty result="bCompileAFSProject" ini="Engine" section="/Script/AndroidFileServerEditor.AndroidFileServerRuntimeSettings" property="bCompileAFSProject" default="false"/>
		<setStringFromProperty result="SecurityToken" ini="Engine" section="/Script/AndroidFileServerEditor.AndroidFileServerRuntimeSettings" property="SecurityToken" default=""/>

		<!-- create the AFS project if plugin enabled -->
		<setBool result="bCreateAFSProject" value="$B(bEnabled)"/>

		<!-- token check may be disabled by clearing the security token field -->
		<setBoolIsEqual result="bTokenOptional" arg1="$S(SecurityToken)" arg2=""/>
		
		<!-- by default embed AFS in APK if plugin enabled -->
		<setBool result="bEmbedAFS" value="$B(bEnabled)"/>

		<!-- by default external start of AFS allowed if plugin enabled -->
		<setBool result="bExternalStartAFS" value="$B(bEnabled)"/>
			 
		<if condition="bEnabled">
			<false>
				<setBool result="bCreateAFSProject" value="false"/>
			</false>
		</if>

		<!-- disable embedding AFS if shipping but "embed in shipping" not checked  -->
		<setBoolIsEqual result="bIsShipping" arg1="$S(Configuration)" arg2="Shipping"/>
		<if condition="bIsShipping">
			<true>
				<if condition="bIncludeInShipping">
					<false>
						<log text="Disabled embedding AFS because shipping"/>
						<setBool result="bEmbedAFS" value="false"/>
						
						<!-- but we need to compile AFSProject to install data if using AFS -->
						<setBool result="bCompileAFSProject" value="$B(bCreateAFSProject)"/>
					</false>
				</if>
				<if condition="bAllowExternalStartInShipping">
					<false>
						<setBool result="bExternalStartAFS" value="false"/>
					</false>
				</if>
			</true>
		</if>
	</init>

	<!-- register the ini keys that modify the packaging -->
	<registerBuildSettings>
		<insertValue value="/Script/AndroidFileServerEditor.AndroidFileServerRuntimeSettings[bEnablePlugin,bAllowNetworkConnection,SecurityToken,bIncludeInShipping,bAllowExternalStartInShipping]"/>
		<insertNewline/>
	</registerBuildSettings>

	<androidManifestUpdates>
		<if condition="bEnabled">
			<true>
				<if condition="bEmbedAFS">
					<true>
						<if condition="bExternalStartAFS">
							<true>
								<!-- runtime permissions for foreground service -->
								<addPermission android:name="android.permission.FOREGROUND_SERVICE"/>
								<addPermission android:name="android.permission.FOREGROUND_SERVICE_DATA_SYNC"/>
								<addPermission android:name="android.permission.POST_NOTIFICATIONS"/>
								
								<addElements tag="application">
									<service android:name="com.epicgames.unreal.RemoteFileManagerService" android:foregroundServiceType="dataSync" android:exported="true" />
									<activity android:name="com.epicgames.unreal.RemoteFileManagerActivity" android:exported="true" android:screenOrientation="sensor" android:theme="@android:style/Theme.NoDisplay">
										<intent-filter>
											<action android:name="com.epicgames.unreal.RemoteFileManager.intent.COMMAND2" />
										</intent-filter>
									</activity>
								</addElements>
							</true>
						</if>

						<!-- allow or disallow network file server -->
						<if condition="bAllowNetworkConnection">
							<true>
								<addElements tag="application">
									<meta-data android:name="com.epicgames.unreal.RemoteFileManager.bAllowNetworkConnection" android:value="true"/>
								</addElements>
							</true>
							<false>
								<addElements tag="application">
									<meta-data android:name="com.epicgames.unreal.RemoteFileManager.bAllowNetworkConnection" android:value="false"/>
								</addElements>
							</false>
						</if>
					</true>
				</if>

				<!-- force bExternalFilesDir to true for all builds -->
				<setBool result="bUpdatedExternal" value="false"/>
				<loopElements tag="meta-data">
					<setStringFromAttribute result="metaName" tag="$" name="android:name" />
					<setBoolIsEqual result="bIsExternal" arg1="$S(metaName)" arg2="com.epicgames.unreal.GameActivity.bUseExternalFilesDir" />
					<if condition="bIsExternal">
						<true>
							<removeAttribute tag="$" name="android:value"/>
							<addAttribute tag="$" name="android:value" value="true"/>
							<setBool result="bUpdatedExternal" value="true"/>
						</true>
					</if>
				</loopElements>
				<if condition="bUpdatedExternal">
					<false>
						<addElements tag="application">
							<meta-data android:name="com.epicgames.unreal.GameActivity.bUseExternalFilesDir" android:value="true"/>
						</addElements>
					</false>
				</if>
			</true>
		</if>
	</androidManifestUpdates>

	<gradleProperties>
		<if condition="bEnabled">
			<true>
			</true>
		</if>
	</gradleProperties>

	<resourceCopies>
		<if condition="bEmbedAFS">
			<true>
				<copyFile src="$S(PluginDir)/Android/java/com/epicgames/unreal/RemoteFileManager.java" dst="$S(BuildDir)/src/com/epicgames/unreal/RemoteFileManager.java" />
				<copyFile src="$S(PluginDir)/Android/java/com/epicgames/unreal/IAndroidFileServerSetup.java" dst="$S(BuildDir)/src/com/epicgames/unreal/IAndroidFileServerSetup.java" />
				<if condition="bExternalStartAFS">
					<true>
						<copyFile src="$S(PluginDir)/Android/java/com/epicgames/unreal/RemoteFileManagerActivity.java" dst="$S(BuildDir)/src/com/epicgames/unreal/RemoteFileManagerActivity.java" />
						<copyFile src="$S(PluginDir)/Android/java/com/epicgames/unreal/RemoteFileManagerService.java" dst="$S(BuildDir)/src/com/epicgames/unreal/RemoteFileManagerService.java" />
					</true>
					<false>
						<isArch arch="arm64-v8a">
							<deleteFiles filespec="arm64/src/com/epicgames/unreal/RemoteFileManagerActivity.java" />
							<deleteFiles filespec="arm64/src/com/epicgames/unreal/RemoteFileManagerService.java" />
						</isArch>
						<isArch arch="x86_64">
							<deleteFiles filespec="x64/src/com/epicgames/unreal/RemoteFileManagerActivity.java" />
							<deleteFiles filespec="x64/src/com/epicgames/unreal/RemoteFileManagerService.java" />
						</isArch>
					</false>
				</if>
			</true>
			<false>
				<isArch arch="arm64-v8a">
					<deleteFiles filespec="arm64/src/com/epicgames/unreal/RemoteFileManager.java" />
					<deleteFiles filespec="arm64/src/com/epicgames/unreal/RemoteFileManagerActivity.java" />
					<deleteFiles filespec="arm64/src/com/epicgames/unreal/RemoteFileManagerService.java" />
					<deleteFiles filespec="arm64/src/com/epicgames/unreal/IAndroidFileServerSetup.java" />
				</isArch>
				<isArch arch="x86_64">
					<deleteFiles filespec="x64/src/com/epicgames/unreal/RemoteFileManager.java" />
					<deleteFiles filespec="x64/src/com/epicgames/unreal/RemoteFileManagerActivity.java" />
					<deleteFiles filespec="x64/src/com/epicgames/unreal/RemoteFileManagerService.java" />
					<deleteFiles filespec="x64/src/com/epicgames/unreal/IAndroidFileServerSetup.java" />
				</isArch>
			</false>
		</if>
	</resourceCopies>

	<gradleCopies>
		<if condition="bCreateAFSProject">
			<true>
				<!-- create AFSStub project -->
				<copyDir src="$S(PluginDir)/../AFSStub" dst="$S(BuildDir)/gradle/AFSProject" />
				<copyDir src="$S(PluginDir)/Android" dst="$S(BuildDir)/gradle/AFSProject/app/src/main" />
				<copyDir src="$S(BuildDir)/gradle/app/src/main/res" dst="$S(BuildDir)/gradle/AFSProject/app/src/main/res" />
				<copyDir src="$S(BuildDir)/gradle/gradle" dst="$S(BuildDir)/gradle/AFSProject/gradle" />
			</true>
		</if>
	</gradleCopies>

	<buildGradleAdditions>
		<if condition="bCompileAFSProject">
			<true>
				<insert>
<![CDATA[
// --- Begin AFSProject ---
tasks.whenTaskAdded { task ->
	if (!System.getProperty("user.dir").contains("AFSProject")) {
		if (task.name == 'assembleRelease') {
			tasks.create(name:'ueAFSProjectAssembleRelease', type: Exec) {
				description "Compile AFSProject for Release"
				workingDir "$rootDir" + "/AFSProject"
				if (System.getProperty('os.name').toLowerCase(Locale.ROOT).contains('windows')) {
					commandLine 'cmd', '/c', 'gradlew.bat', 'app:assembleRelease'
				} else {
					commandLine 'sh', '-c', 'chmod +x ./gradlew && ./gradlew app:assembleRelease'
				}
			}
			task.finalizedBy 'ueAFSProjectAssembleRelease'
		}
		if (task.name == 'assembleDebug') {
			tasks.create(name:'ueAFSProjectAssembleDebug', type: Exec) {
				description "Compile AFSProject for Debug"
				workingDir "$rootDir" + "/AFSProject"
				if (System.getProperty('os.name').toLowerCase(Locale.ROOT).contains('windows')) {
					commandLine 'cmd', '/c', 'gradlew.bat', 'app:assembleDebug'
				} else {
					commandLine 'sh', '-c', 'chmod +x ./gradlew && ./gradlew app:assembleDebug'
				}
			}
			task.finalizedBy 'ueAFSProjectAssembleDebug'
		}
		if (task.name == 'bundleRelease') {
			tasks.create(name:'ueAFSProjectBundleRelease', type: Exec) {
				description "Compile AFSProject for Release"
				workingDir "$rootDir" + "/AFSProject"
				if (System.getProperty('os.name').toLowerCase(Locale.ROOT).contains('windows')) {
					commandLine 'cmd', '/c', 'gradlew.bat', 'app:assembleRelease'
				} else {
					commandLine 'sh', '-c', 'chmod +x ./gradlew && ./gradlew app:assembleRelease'
				}
			}
			task.finalizedBy 'ueAFSProjectBundleRelease'
		}
		if (task.name == 'bundleDebug') {
			tasks.create(name:'ueAFSProjectBundleDebug', type: Exec) {
				description "Compile AFSProject for Debug"
				workingDir "$rootDir" + "/AFSProject"
				if (System.getProperty('os.name').toLowerCase(Locale.ROOT).contains('windows')) {
					commandLine 'cmd', '/c', 'gradlew.bat', 'app:assembleDebug'
				} else {
					commandLine 'sh', '-c', 'chmod +x ./gradlew && ./gradlew app:assembleDebug'
				}
			}
			task.finalizedBy 'ueAFSProjectBundleDebug'
		}
	}
}
// --- End AFSProject ---
]]></insert>
			</true>
		</if>
	</buildGradleAdditions>
	
	<settingsGradleAdditions>
		<if condition="bCreateAFSProject">
			<true>
				<insert>
<![CDATA[
	// copy files into AFSStub
	File afs_infile = new File(rootProject.projectDir, 'gradle.properties')
	File afs_outfile = new File(rootProject.projectDir, 'AFSProject/gradle.properties')
	def afs_line = "";
	afs_outfile.withWriter { writer ->
		afs_infile.withReader { reader ->
			while ((afs_line = reader.readLine()) != null) {
				if (afs_line.startsWith('OUTPUT_FILENAME=')) {
					writer.writeLine("OUTPUT_FILENAME=AFS_" + afs_line.substring(16));
					continue;
				}
				else if (afs_line.startsWith('OUTPUT_BUNDLEFILENAME=')) {
					writer.writeLine("OUTPUT_BUNDLEFILENAME=AFS_" + afs_line.substring(22));
					continue;
				}
				writer.writeLine("${afs_line}")
			}
		}
	}

	afs_infile = new File(rootProject.projectDir, 'app/buildAdditions.gradle')
	afs_outfile = new File(rootProject.projectDir, 'AFSProject/app/buildAdditions.gradle')
	afs_line = "";
	def bRemovingAFSBlock = false;
	afs_outfile.withWriter { writer ->
		afs_infile.withReader { reader ->
			while ((afs_line = reader.readLine()) != null) {
				if (bRemovingAFSBlock)
				{
					if (afs_line.startsWith('// --- End AFSProject ---'))
					{
						bRemovingAFSBlock = false;
					}
					continue;
				}
				if (afs_line.startsWith('// --- Begin AFSProject ---'))
				{
					bRemovingAFSBlock = true;
					continue;
				}
				writer.writeLine("${afs_line}")
			}
		}
	}

	afs_infile = new File(rootProject.projectDir, 'app/src/main/AndroidManifest.xml')
	afs_outfile = new File(rootProject.projectDir, 'AFSProject/app/src/main/AndroidManifest.xml')
	def afs_hasForeServiceDataPerm = false;
	def afs_hasForeServicePerm = false;
	def afs_hasNotificationsPerm = false;
	def afs_hasService = false
	def afs_hasActivity = false
	afs_outfile.withWriter { writer ->
		afs_infile.withReader { reader ->
			while ((afs_line = reader.readLine()) != null) {
				if (afs_line.contains('__ExcludeFromAFS="true"')) {
					continue;
				}
				if (afs_line.contains('android.permission.FOREGROUND_SERVICE_DATA_SYNC')) {
					afs_hasForeServiceDataPerm = true
				} else if (afs_line.contains('android.permission.FOREGROUND_SERVICE')) {
					afs_hasForeServicePerm = true
				} else if (afs_line.contains('android.permission.POST_NOTIFICATIONS')) {
					afs_hasNotificationsPerm = true
				} else if (afs_line.contains('com.epicgames.unreal.RemoteFileManagerService')) {
					afs_hasService = true
				} else if (afs_line.contains('com.epicgames.unreal.RemoteFileManagerActivity')) {
					afs_hasActivity = true
				} else if (afs_line.contains('</application>')) {
					if (afs_hasService == false) {
						writer.writeLine('    <service android:name="com.epicgames.unreal.RemoteFileManagerService" android:foregroundServiceType="dataSync" android:exported="true" />')
					}
					if (afs_hasActivity == false) {
						writer.writeLine('    <activity android:name="com.epicgames.unreal.RemoteFileManagerActivity" android:exported="true" android:screenOrientation="sensor">')
						writer.writeLine('    	<intent-filter>')
						writer.writeLine('    		<action android:name="com.epicgames.unreal.RemoteFileManager.intent.COMMAND2" />')
						writer.writeLine('    	</intent-filter>')
						writer.writeLine('    </activity>')
					}
				} else if (afs_line.contains('</manifest>')) {
					if (afs_hasForeServicePerm == false) {
						writer.writeLine('  <uses-permission android:name="android.permission.FOREGROUND_SERVICE" />')
					}
					if (afs_hasForeServiceDataPerm == false) {
						writer.writeLine('  <uses-permission android:name="android.permission.FOREGROUND_SERVICE_DATA_SYNC" />')
					}
					if (afs_hasNotificationsPerm == false) {
						writer.writeLine('  <uses-permission android:name="android.permission.POST_NOTIFICATIONS" />')
					}
				}
				writer.writeLine("${afs_line}")
			}
		}
	}

	def afs_files = ["app/aar-imports.gradle", "app/abi.gradle", "app/build.gradle",
		"baseBuildAdditions.gradle", "build.gradle", "buildscriptAdditions.gradle", "local.properties",
		"gradlew", "gradlew.bat", "rungradle.bat"] as String[]
	afs_files.eachWithIndex { v, i ->
		afs_infile = new File(rootProject.projectDir, v)
		if (afs_infile.exists()) {
			afs_outfile = new File(rootProject.projectDir, "AFSProject/" + v)
			afs_outfile.withWriter { writer ->
				afs_infile.withReader { reader ->
					while ((afs_line = reader.readLine()) != null) {
						writer.writeLine("${afs_line}")
					}
				}
			}
		}
	}

]]></insert>
			</true>
		</if>
	</settingsGradleAdditions>

	<gameApplicationImplementsAdditions>
		<if condition="bEmbedAFS">
			<true>
				<insert>, IAndroidFileServerSetup </insert>
			</true>
		</if>
	</gameApplicationImplementsAdditions>

	<gameApplicationImportAdditions>
		<if condition="bEmbedAFS">
			<true>
				<insert>
import java.io.BufferedReader;
import java.io.File;
import java.io.FileReader;
import java.io.FileWriter;
import java.io.IOException;
import com.epicgames.unreal.IAndroidFileServerSetup;
				</insert>
			</true>
		</if>
	</gameApplicationImportAdditions>

	<gameActivityImportAdditions>
		<if condition="bEmbedAFS">
			<true>
				<insert>
import java.util.Set;
import com.epicgames.unreal.RemoteFileManager;
				</insert>
			</true>
		</if>
	</gameActivityImportAdditions>

	<gameApplicationClassAdditions>
		<if condition="bEmbedAFS">
			<true>
				<if condition="Distribution">
					<true>
						<insert>
	public boolean AndroidFileServer_Init(String filename)
	{
		return false;
	}
						</insert>
					</true>
					<false>
						<insert>
	public boolean AndroidFileServer_Init(String filename)
	{
		File file = new File(filename);
		try
		{
			boolean bNeedsWrite = true;
			if (file.exists())
			{
				String Token = "";
				BufferedReader reader = null;
				try
				{
					reader = new BufferedReader(new FileReader(file));
					Token = reader.readLine();
					reader.close();
						</insert>
						<insertValue value="&#9;&#9;&#9;&#9;&#9;if (&quot;$S(SecurityToken)&quot;.equals(Token))"/>
						<insertNewline/>
						<insert>
					{
						bNeedsWrite = false;
					}
				}
				catch (IOException ie)
				{
				}
			}
			if (bNeedsWrite)
			{

				FileWriter outputWriter = new FileWriter(file);
						</insert>
						<insertValue value="&#9;&#9;&#9;&#9;outputWriter.write(&quot;$S(SecurityToken)&quot;);"/>
						<insertNewline/>
						<insert>
				outputWriter.close();
			}
		}
		catch (IOException e)
		{
			return false;
		}
		return true;
	}
						</insert>
					</false>
				</if>
				<insertNewline/>
				<insert>
	public boolean AndroidFileServer_Verify(String Token)
	{
				</insert>
				<if condition="bTokenOptional">
					<true>
						<insertValue value="&#9;&#9;return true;"/>
					</true>
					<false>
						<insertValue value="&#9;&#9;return (&quot;$S(SecurityToken)&quot;.equals(Token));"/>
					</false>
				</if>
				<insertNewline/>
				<insert>
	}
				</insert>
			</true>
			<false>
				<insert>
	public boolean AndroidFileServer_Init(String filename)
	{
		return true;
	}

	public boolean AndroidFileServer_Verify(String Token)
	{
		return false;
	}
				</insert>
			</false>
		</if>
	</gameApplicationClassAdditions>

	<gameActivityClassAdditions>
		<if condition="bEmbedAFS">
			<true>
				<insert>
<![CDATA[
	public int AndroidThunkJava_AndroidFileServer_IsRunning()
	{
        Set<Thread> threads = Thread.getAllStackTraces().keySet();
		int ConnectionType = 0;
        for (Thread thread : threads) {
            String name = thread.getName();
            if (name.equals("UnrealAFS-USB"))
            {
                if (thread.isAlive()) {
					ConnectionType |= 1;
                }
            }
            else if (name.equals("UnrealAFS-WiFi"))
            {
                if (thread.isAlive()) {
					ConnectionType |= 2;
                }
            }
        }
		
		return ConnectionType;
	}

	public boolean AndroidThunkJava_AndroidFileServer_Stop(boolean bUSB, boolean bNetwork)
	{
        Set<Thread> threads = Thread.getAllStackTraces().keySet();
        boolean bFoundThread = false;
        for (Thread thread : threads) {
            String name = thread.getName();
            if (bUSB && name.equals("UnrealAFS-USB"))
            {
                if (thread.isAlive()) {
					thread.interrupt();
                    bFoundThread = true;
                }
            }
            else if (bNetwork && name.equals("UnrealAFS-WiFi"))
            {
                if (thread.isAlive()) {
					thread.interrupt();
                    bFoundThread = true;
                }
            }
        }
		
		return bFoundThread;
	}

	public boolean AndroidThunkJava_AndroidFileServer_Start(boolean bUSB, boolean bNetwork, int port)
	{
		boolean bUSBRunning = false;
		boolean bWiFiRunning = false;
		
		if (port == 0)
		{
			port = 57099;
		}
		
        Set<Thread> threads = Thread.getAllStackTraces().keySet();
        boolean bFoundThread = false;
        for (Thread thread : threads) {
            String name = thread.getName();
            if (name.equals("UnrealAFS-USB"))
            {
                if (thread.isAlive()) {
                    bUSBRunning = true;
                }
            }
            else if (name.equals("UnrealAFS-WiFi"))
            {
                if (thread.isAlive()) {
                    bWiFiRunning = true;
                    break;
                }
            }
        }

		if (bUSBRunning && bWiFiRunning)
		{
			return true;
		}

		String packageName = getPackageName();
		PackageManager pm = getPackageManager();
		
		int versionCode = 0;
		try
		{
			versionCode = pm.getPackageInfo(packageName, 0).versionCode;
		}
		catch (Exception e)
		{
			// if the above failed, then, we can't use obbs
		}

		String EngineVersion = "5.0.0";
		String ProjectName = packageName;
		ProjectName = ProjectName.substring(ProjectName.lastIndexOf('.') + 1);
		boolean IsShipping = IS_SHIPPING_CONFIG;
		boolean UseExternalFilesDir = false;
		boolean PublicLogFiles = false;
		boolean AllowNetworkConnection = false;
		try {
			ApplicationInfo ai = pm.getApplicationInfo(packageName, PackageManager.GET_META_DATA);
			Bundle bundle = ai.metaData;

			if (bundle.containsKey("com.epicgames.unreal.GameActivity.EngineVersion"))
			{
				EngineVersion = bundle.getString("com.epicgames.unreal.GameActivity.EngineVersion");
			}
			if (bundle.containsKey("com.epicgames.unreal.GameActivity.ProjectName"))
			{
				ProjectName = bundle.getString("com.epicgames.unreal.GameActivity.ProjectName");
			}
			if (bundle.containsKey("com.epicgames.unreal.GameActivity.bUseExternalFilesDir"))
			{
				UseExternalFilesDir = bundle.getBoolean("com.epicgames.unreal.GameActivity.bUseExternalFilesDir");
			}
			if (bundle.containsKey("com.epicgames.unreal.GameActivity.bPublicLogFiles"))
			{
				PublicLogFiles = bundle.getBoolean("com.epicgames.unreal.GameActivity.bPublicLogFiles");
			}
			if (bundle.containsKey("com.epicgames.unreal.RemoteFileManager.bAllowNetworkConnection"))
			{
				AllowNetworkConnection = bundle.getBoolean("com.epicgames.unreal.RemoteFileManager.bAllowNetworkConnection");
			}
		}
		catch (PackageManager.NameNotFoundException | NullPointerException e)
		{
			Log.debug("Error when accessing application metadata");
		}

		String internal = getFilesDir().getAbsolutePath();
		String external = getExternalFilesDir(null).getAbsolutePath();
		String storage = android.os.Environment.getExternalStorageDirectory().getAbsolutePath();
		String obbdir = getObbDir().getAbsolutePath();

		if (bUSB && !bUSBRunning)
		{
			RemoteFileManager fileManager = new RemoteFileManager(true, port, internal, external, storage, obbdir, packageName, versionCode, ProjectName, EngineVersion, IsShipping, PublicLogFiles);
			new Thread(fileManager, "UnrealAFS-USB").start();
		}
		if (AllowNetworkConnection && bNetwork && !bWiFiRunning)
		{
			RemoteFileManager fileManager = new RemoteFileManager(false, port, internal, external, storage, obbdir, packageName, versionCode, ProjectName, EngineVersion, IsShipping, PublicLogFiles);
			new Thread(fileManager, "UnrealAFS-WiFi").start();
		}
		
		return true;
	}
]]>
				</insert>
			</true>
			<false>
	public boolean AndroidThunkJava_AndroidFileServer_IsRunning()
	{ 
		return false;
	}

	public boolean AndroidThunkJava_AndroidFileServer_Start()
	{
		return false;
	}

	public boolean AndroidThunkJava_AndroidFileServer_Stop()
	{
		return false;
	}
			</false>
		</if>
	</gameActivityClassAdditions>

</root>
