// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"
#include "AndroidFileServerBPLibrary.generated.h"

UENUM()
namespace EAFSActiveType
{
	enum Type : int
	{
		None = 0 UMETA(DisplayName = "None"),
		USBOnly = 1 UMETA(DisplayName = "USB only"),
		NetworkOnly = 2 UMETA(DisplayName = "Network only"),
		Combined = 3 UMETA(DisplayName = "USB and Network combined"),
	};
}

UCLASS()
class UAndroidFileServerBPLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()
public:
	/** Request startup of Android FileServer */
	UFUNCTION(BlueprintCallable, Category = "AndroidFileServer")
	static bool StartFileServer(bool bUSB = true, bool bNetwork = false, int32 Port = 57099);

	/** Request termination of Android FileServer */
	UFUNCTION(BlueprintCallable, Category = "AndroidFileServer")
	static bool StopFileServer(bool bUSB = true, bool bNetwork = true);

	/** Check if Android FileServer is running */
	UFUNCTION(BlueprintCallable, Category = "AndroidFileServer")
	static TEnumAsByte<EAFSActiveType::Type> IsFileServerRunning();
};
