// Copyright Epic Games, Inc. All Rights Reserved.

#include "AndroidFileServerBPLibrary.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AndroidFileServerBPLibrary)

#if PLATFORM_ANDROID

#include "Android/AndroidJNI.h"
#include "Android/AndroidApplication.h"

#if USE_ANDROID_JNI
#include <jni.h>
#endif
#endif

bool UAndroidFileServerBPLibrary::StartFileServer(bool bUSB, bool bNetwork, int32 Port)
{
	bool result = false;
#if PLATFORM_ANDROID
	if (JNIEnv* Env = FAndroidApplication::GetJavaEnv())
	{
		static jmethodID StartFunc = FJavaWrapper::FindMethod(Env, FJavaWrapper::GameActivityClassID, "AndroidThunkJava_AndroidFileServer_Start", "(ZZI)Z", false);
		if (StartFunc != nullptr)
		{
			result = FJavaWrapper::CallBooleanMethod(Env, FJavaWrapper::GameActivityThis, StartFunc, bUSB, bNetwork, Port);
		}
	}
#endif
	return result;
}

bool UAndroidFileServerBPLibrary::StopFileServer(bool bUSB, bool bNetwork)
{
	bool result = false;
#if PLATFORM_ANDROID
	if (JNIEnv* Env = FAndroidApplication::GetJavaEnv())
	{
		static jmethodID StopFunc = FJavaWrapper::FindMethod(Env, FJavaWrapper::GameActivityClassID, "AndroidThunkJava_AndroidFileServer_Stop", "(ZZ)Z", false);
		if (StopFunc != nullptr)
		{
			result = FJavaWrapper::CallBooleanMethod(Env, FJavaWrapper::GameActivityThis, StopFunc, bUSB, bNetwork);
		}
	}
#endif
	return result;
}

TEnumAsByte<EAFSActiveType::Type> UAndroidFileServerBPLibrary::IsFileServerRunning()
{
	TEnumAsByte<EAFSActiveType::Type> result = EAFSActiveType::None;
#if PLATFORM_ANDROID
	if (JNIEnv* Env = FAndroidApplication::GetJavaEnv())
	{
		static jmethodID IsRunningFunc = FJavaWrapper::FindMethod(Env, FJavaWrapper::GameActivityClassID, "AndroidThunkJava_AndroidFileServer_IsRunning", "()I", false);
		if (IsRunningFunc != nullptr)
		{
			result = (TEnumAsByte<EAFSActiveType::Type>)FJavaWrapper::CallIntMethod(Env, FJavaWrapper::GameActivityThis, IsRunningFunc);
		}
	}
#endif
	return result;
}


