// Copyright Epic Games, Inc. All Rights Reserved.

#include "AndroidFileServerEditor.h"
#include "ISettingsModule.h"
#include "AndroidFileServerRuntimeSettings.h"
#include "Modules/ModuleManager.h"

/**
 * Implements the AndroidFileServerEditor module.
 */

#define LOCTEXT_NAMESPACE "AndroidFileServer"

void FAndroidFileServerEditorModule::StartupModule()
{
	// register settings
	ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings");

	if (SettingsModule != nullptr)	
	{
		SettingsModule->RegisterSettings("Project", "Plugins", "AndroidFileServer",
			LOCTEXT("AndroidFileServerSettingsName", "AndroidFileServer"),
			LOCTEXT("AndroidFileServerSettingsDescription", "Project settings for AndroidFileServer plugin"),
			GetMutableDefault<UAndroidFileServerRuntimeSettings>()
		);
	}
}

void FAndroidFileServerEditorModule::ShutdownModule()
{
	// unregister settings
	ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings");

	if (SettingsModule != nullptr)
	{
       		SettingsModule->UnregisterSettings("Project", "Plugins", "AndroidFileServer");
	}
}


IMPLEMENT_MODULE(FAndroidFileServerEditorModule, AndroidFileServerEditor);

#undef LOCTEXT_NAMESPACE
