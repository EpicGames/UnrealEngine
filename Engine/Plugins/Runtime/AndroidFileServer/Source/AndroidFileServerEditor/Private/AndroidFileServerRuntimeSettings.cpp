// Copyright Epic Games, Inc. All Rights Reserved.

#include "AndroidFileServerRuntimeSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AndroidFileServerRuntimeSettings)

//////////////////////////////////////////////////////////////////////////
// UAndroidFileServerRuntimeSettings

UAndroidFileServerRuntimeSettings::UAndroidFileServerRuntimeSettings(const FObjectInitializer& ObjectInitializer)
        : Super(ObjectInitializer)
        , bEnablePlugin(true)
		, bAllowNetworkConnection(true)
		, SecurityToken(TEXT("[AUTO]"))
		, bIncludeInShipping(false)
		, bAllowExternalStartInShipping(false)
		, bCompileAFSProject(false)
		, bUseCompression(false)
		, bLogFiles(false)
		, bReportStats(false)
		, ConnectionType(EAFSConnectionType::USBOnly)
		, bUseManualIPAddress(false)
		, ManualIPAddress(TEXT(""))
{
}

void UAndroidFileServerRuntimeSettings::PostInitProperties()
{
	Super::PostInitProperties();

	// Auto-generate a security token first time
	if (SecurityToken == "[AUTO]")
	{
		FGuid UniqueId = FGuid::NewGuid();
		SecurityToken = UniqueId.ToString();
		TryUpdateDefaultConfigFile(TEXT(""), false);
	}
}
