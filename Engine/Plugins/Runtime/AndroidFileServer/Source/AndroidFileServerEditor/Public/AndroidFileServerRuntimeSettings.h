// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AndroidFileServerRuntimeSettings.generated.h"

/**
* Implements the settings for the AndroidFileServer plugin.
*/

UENUM()
namespace EAFSConnectionType
{
	enum Type : int
	{
		USBOnly = 0 UMETA(DisplayName = "USB only"),
		NetworkOnly = 1 UMETA(DisplayName = "Network only"),
		Combined = 2 UMETA(DisplayName = "USB and Network combined"),
	};
}


UCLASS(Config = Engine, DefaultConfig)
class UAndroidFileServerRuntimeSettings : public UObject
{
	GENERATED_UCLASS_BODY()

	// Enable Android FileServer for packaged builds and quick launch
	UPROPERTY(EditAnywhere, config, Category = Packaging, Meta = (DisplayName = "Use AndroidFileServer"))
	bool bEnablePlugin;

	// Allow FileServer connection using network
	UPROPERTY(EditAnywhere, config, Category = Packaging, Meta = (EditCondition = "bEnablePlugin"))
	bool bAllowNetworkConnection;

	// Optional security token required to start FileServer (leave empty to disable)
	UPROPERTY(EditAnywhere, config, Category = Packaging, Meta = (EditCondition = "bEnablePlugin"))
	FString SecurityToken;

	// Embed FileServer in Shipping builds
	UPROPERTY(EditAnywhere, config, Category = Packaging, Meta=(EditCondition = "bEnablePlugin"))
	bool bIncludeInShipping;

	// Allow FileServer to be started in Shipping builds with UnrealAndroidFileTool
	UPROPERTY(EditAnywhere, config, Category = Packaging, Meta = (EditCondition = "bEnablePlugin && bIncludeInShipping"))
	bool bAllowExternalStartInShipping;

	// Compile standalone AFS project
	UPROPERTY(EditAnywhere, config, Category = Packaging, Meta=(EditCondition = "bEnablePlugin"))
	bool bCompileAFSProject;

	// Enable compression during data transfer
	UPROPERTY(EditAnywhere, config, Category = Deployment, Meta=(EditCondition = "bEnablePlugin"))
	bool bUseCompression;

	// Log files transferred
	UPROPERTY(EditAnywhere, config, Category = Deployment, Meta=(EditCondition = "bEnablePlugin"))
	bool bLogFiles;

	// Report transfer rate statistics
	UPROPERTY(EditAnywhere, config, Category = Deployment, Meta=(EditCondition = "bEnablePlugin"))
	bool bReportStats;

	// How to connect to file server (USB cable, Network, or combined)
	UPROPERTY(EditAnywhere, config, Category = Connection, Meta=(EditCondition = "bEnablePlugin"))
	TEnumAsByte<EAFSConnectionType::Type> ConnectionType;

	// Use manual IP address instead of automatic query from device (only for single device deploys!)
	UPROPERTY(EditAnywhere, config, Category = Connection, Meta=(EditCondition = "ConnectionType == EAFSConnectionType::NetworkOnly || ConnectionType == EAFSConnectionType::Combined"), Meta=(DisplayName = "Use Manual IP Address?"))
	bool bUseManualIPAddress;

	// IP address of device to use
	UPROPERTY(EditAnywhere, config, Category = Connection, Meta=(EditCondition = "bUseManualIPAddress"), Meta=(DisplayName = "Manual IP Address"))
	FString ManualIPAddress;

private:
	// UObject interface
	virtual void PostInitProperties() override;
};
